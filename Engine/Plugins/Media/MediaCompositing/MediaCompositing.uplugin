{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Media Compositing",
	"Description" : "Actors, components and Sequencer extensions for compositing media",
	"Category" : "Rendering",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : true,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "MediaCompositing",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "MediaCompositingEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		}
	],
	"Plugins": 
	[
		{
			"Name": "ImgMedia",
			"TargetAllowList": [ "Editor" ],
			"Enabled": true
		}
	]
}
