// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MediaCompositing : ModuleRules
	{
		public MediaCompositing(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"MediaUtils",
					"MovieScene",
					"MovieSceneTracks",
					"TimeManagement",
				});

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("UnrealEd");
			}

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Media",
					"MediaAssets",
					"RenderCore",
					"RHI",
				});
		}
	}
}
