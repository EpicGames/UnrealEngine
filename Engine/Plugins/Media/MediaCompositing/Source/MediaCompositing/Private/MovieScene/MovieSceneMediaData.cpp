// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneMediaData.h"

#include "IMediaEventSink.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "MediaPlayerProxyInterface.h"
#include "MediaTexture.h"
#include "MovieSceneMediaPlayerStore.h"
#include "MovieSceneMediaPlayerUtils.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

/* FMediaSectionData structors
 *****************************************************************************/

FMovieSceneMediaData::FMovieSceneMediaData()
	: bIsAspectRatioSet(false)
	, bOverrideMediaPlayer(false)
	, MediaPlayer(nullptr)
	, ProxyLayerIndex(0)
	, ProxyTextureIndex(0)
	, SeekOnOpenTime(FTimespan::MinValue())
{ }


FMovieSceneMediaData::~FMovieSceneMediaData()
{
	if (MediaPlayer != nullptr)
	{
		MediaPlayer->OnMediaEvent().RemoveAll(this);

		if (const TSharedPtr<FMovieSceneMediaPlayerStore> MediaPlayerStore = MediaPlayerStoreWeak.Pin())
		{
			MediaPlayerStore->ScheduleMediaPlayerForRelease(MediaSection, MediaPlayer);
		}
		else
		{
			// If we don't have a store (ex: external player), close media player immediately.
			FMovieSceneMediaPlayerStore::CloseMediaPlayer(MediaPlayer, !bOverrideMediaPlayer);
		}
	}
}


/* FMediaSectionData interface
 *****************************************************************************/

UMediaPlayer* FMovieSceneMediaData::GetMediaPlayer()
{
	return MediaPlayer;
}


void FMovieSceneMediaData::SeekOnOpen(FTimespan Time)
{
	SeekOnOpenTime = Time;
}

void FMovieSceneMediaData::Setup(const TSharedPtr<FMovieSceneMediaPlayerStore>& InMediaPlayerStore, const UObject* InSection, UMediaPlayer* OverrideMediaPlayer, UObject* InPlayerProxy, int32 InProxyLayerIndex, int32 InProxyTextureIndex)
{
	Setup(InMediaPlayerStore, InSection, OverrideMediaPlayer, InPlayerProxy, InProxyLayerIndex, InProxyTextureIndex, FMovieSceneMediaPlaybackParams());
}

void FMovieSceneMediaData::Setup(const TSharedPtr<FMovieSceneMediaPlayerStore>& InMediaPlayerStore, const UObject* InMediaSection, UMediaPlayer* OverrideMediaPlayer, UObject* InPlayerProxy, int32 InProxyLayerIndex, int32 InProxyTextureIndex, const FMovieSceneMediaPlaybackParams& InPlaybackParams)
{
	// Ensure we don't already have a media player set. Setup should only be called once
	check(!MediaPlayer);

	if (OverrideMediaPlayer)
	{
		MediaPlayer = OverrideMediaPlayer;
		bOverrideMediaPlayer = true;
	}
	else if (MediaPlayer == nullptr)
	{
		MediaPlayerStoreWeak = InMediaPlayerStore;
		MediaSection = InMediaSection;
		// Try to acquire a player from the store associated to the owning section.
		MediaPlayer = InMediaPlayerStore ? InMediaPlayerStore->TryAcquireMediaPlayer(InMediaSection) : nullptr;
		if (!MediaPlayer)
		{
			MediaPlayer = NewObject<UMediaPlayer>(GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UMediaPlayer::StaticClass()));
		}
	}

	MediaPlayer->PlayOnOpen = false;
	MediaPlayer->OnMediaEvent().AddRaw(this, &FMovieSceneMediaData::HandleMediaPlayerEvent);
	MediaPlayer->AddToRoot();
	ProxyMediaTexture.Reset();
	ProxyLayerIndex = InProxyLayerIndex;
	ProxyTextureIndex = InProxyTextureIndex;
	PlaybackParams = InPlaybackParams;

	UpdatePlayerProxy(InPlayerProxy);

	SampleQueue = MakeShared<FMediaTextureSampleQueue>();
	MediaPlayer->GetPlayerFacade()->AddVideoSampleSink(SampleQueue.ToSharedRef());
}

void FMovieSceneMediaData::UpdatePlayerProxy(UObject* InPlayerProxy)
{
	UObject* PreviousPlayerProxy = PlayerProxy.Get();

	// Do we have a valid proxy object?
	if ((InPlayerProxy != nullptr) && (InPlayerProxy->Implements<UMediaPlayerProxyInterface>()))
	{
		PlayerProxy = InPlayerProxy;
	}
	else
	{
		PlayerProxy.Reset();
	}

	// If the player proxy is changed, we need to reset the cached proxied media texture so it is
	// refreshed from the new proxy in the next evaluation.
	if ((PlayerProxy == nullptr) || (PlayerProxy != PreviousPlayerProxy))
	{
		ProxyMediaTexture.Reset();
	}
}

void FMovieSceneMediaData::Initialize(bool bIsEvaluating)
{
	if (bIsEvaluating)
	{
		StartUsingProxyMediaTexture();
	}
	else
	{
		StopUsingProxyMediaTexture();
	}
}

void FMovieSceneMediaData::TearDown()
{
	StopUsingProxyMediaTexture();
}

TSharedPtr<FMediaTextureSampleQueue> FMovieSceneMediaData::TransferSampleQueue()
{
	TSharedPtr<FMediaTextureSampleQueue> OutSampleQueue = SampleQueue;
	SampleQueue.Reset();
	return OutSampleQueue;
}

UMediaTexture* FMovieSceneMediaData::GetProxyMediaTexture()
{
	if (PlayerProxy != nullptr)
	{
		if (ProxyMediaTexture == nullptr)
		{
			IMediaPlayerProxyInterface* PlayerProxyInterface = Cast<IMediaPlayerProxyInterface>(PlayerProxy);
			if (PlayerProxyInterface != nullptr)
			{
				ProxyMediaTexture = PlayerProxyInterface->ProxyGetMediaTexture(ProxyLayerIndex, ProxyTextureIndex);
			}
		}
	}
	return ProxyMediaTexture.Get();
}

void FMovieSceneMediaData::StartUsingProxyMediaTexture()
{
	if (UMediaTexture* MediaTexture = GetProxyMediaTexture())
	{
		MediaTexture->SetMediaPlayer(MediaPlayer, TransferSampleQueue());
	}
}

void FMovieSceneMediaData::StopUsingProxyMediaTexture()
{
	if (PlayerProxy != nullptr)
	{
		if (ProxyMediaTexture != nullptr)
		{
			if (ProxyMediaTexture->GetMediaPlayer() == MediaPlayer)
			{
				ProxyMediaTexture->SetMediaPlayer(nullptr);
			}
			IMediaPlayerProxyInterface* PlayerProxyInterface = Cast<IMediaPlayerProxyInterface>(PlayerProxy);
			if (PlayerProxyInterface != nullptr)
			{
				PlayerProxyInterface->ProxyReleaseMediaTexture(ProxyLayerIndex, ProxyTextureIndex);
			}
			ProxyMediaTexture = nullptr;
		}
	}
}

/* FMediaSectionData callbacks
 *****************************************************************************/

void FMovieSceneMediaData::HandleMediaPlayerEvent(EMediaEvent Event)
{
	if ((Event != EMediaEvent::MediaOpened) || (SeekOnOpenTime < FTimespan::Zero()))
	{
		return; // we only care about seek on open
	}

	if (!MediaPlayer->SupportsSeeking())
	{
		return; // media can't seek
	}

	const FTimespan Duration = MediaPlayer->GetDuration();
	if (Duration == FTimespan::Zero())
	{
		return;
	}

	// Set looping from the media section parameter.
	// Remark: this must be set to the value it is going to be in the media section, switching it will cause a seek/flush on Electra.
	MediaPlayer->SetLooping(PlaybackParams.bIsLooping);

	FTimespan MediaTime;

	if (PlaybackParams.bIsLooping)
	{
		MediaTime = SeekOnOpenTime % Duration;
	}
	else
	{
		MediaTime = SeekOnOpenTime;
	}

	const FTimespan ClampTolerance = PlaybackParams.FrameDuration * 0.5f;
	MediaTime = FMath::Clamp(MediaTime, FTimespan::Zero(), Duration - ClampTolerance);

	if (!PlaybackParams.SectionTimeRange.IsEmpty() && MediaPlayer->SupportsPlaybackTimeRange())
	{
		using namespace UE::MovieSceneMediaPlayerUtils;
		const TRange<FTimespan> AdjustedRange = AdjustPlaybackTimeRange(PlaybackParams.SectionTimeRange, MediaPlayer, PlaybackParams.FrameDuration);

		// We can only set the player's playback time range if the requested seek time is contained within.
		// It is possible the media is opened by scrubbing or stepping outside the playback range.
		if (!AdjustedRange.IsEmpty() && AdjustedRange.Contains(MediaTime))
		{
			SetPlayerPlaybackTimeRange(MediaPlayer, AdjustedRange);
		}
	}

	MediaPlayer->SetRate(0.0f);
	MediaPlayer->Seek(MediaTime);
	
	SeekOnOpenTime = FTimespan::MinValue();
}
