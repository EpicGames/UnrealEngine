// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneMediaPlayerPropertySection.h"
#include "MovieScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMediaPlayerPropertySection)

UMovieSceneMediaPlayerPropertySection::UMovieSceneMediaPlayerPropertySection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	EvalOptions.CompletionMode = EMovieSceneCompletionMode::RestoreState;
	bLoop = false;

	UMovieScene* Outer = GetTypedOuter<UMovieScene>();
	FFrameRate TickResolution = Outer ? Outer->GetTickResolution() : FFrameRate(24, 1);

	SetPreRollFrames( (0.5 * TickResolution).RoundToFrame().Value );
}

