// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneMediaPlayerPropertyTemplate.h"

#include "MediaPlayer.h"
#include "MediaSource.h"
#include "MediaPlaylist.h"

#include "MovieSceneMediaPlayerPropertySection.h"
#include "MovieSceneMediaPlayerPropertyTrack.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMediaPlayerPropertyTemplate)


struct FMediaPlayerPreAnimatedState : IMovieScenePreAnimatedToken
{
	virtual void RestoreState(UObject& Object, const UE::MovieScene::FRestoreStateParams& Params)
	{
		CastChecked<UMediaPlayer>(&Object)->Close();
	}
};

struct FMediaPlayerSectionPreRollExecutionToken : IMovieSceneExecutionToken
{
	FMediaPlayerSectionPreRollExecutionToken(UMediaSource* InMediaSource, FTimespan InStartTimeSeconds, bool bInLoop)
		: MediaSource(InMediaSource)
		, StartTime(InStartTimeSeconds)
		, bLoop(bInLoop)
	{}

	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		using namespace PropertyTemplate;
		const FSectionData& SectionData = PersistentData.GetSectionData<FSectionData>();

		const FTimespan TwoFrames = FTimespan::FromSeconds(1 / 12.f);

		for (TWeakObjectPtr<> WeakObject : Player.FindBoundObjects(Operand))
		{
			UObject* BoundObject = WeakObject.Get();
			if (!BoundObject)
			{
				continue;
			}

			UMediaPlayer* MediaPlayer = SectionData.PropertyBindings->GetCurrentValue<UMediaPlayer*>(*BoundObject);
			if (!MediaPlayer)
			{
				continue;
			}

			// Ensure the media player is playing this media
			const int32 PlaylistIndex = MediaPlayer->GetPlaylistIndex();
			if (PlaylistIndex == INDEX_NONE || MediaPlayer->GetPlaylistRef().Get(PlaylistIndex) != MediaSource)
			{
				FMediaPlayerOptions Options;
				Options.SeekTime   = StartTime;
				Options.PlayOnOpen = EMediaPlayerOptionBooleanOverride::Disabled;
				Options.Loop       = bLoop ? EMediaPlayerOptionBooleanOverride::Enabled : EMediaPlayerOptionBooleanOverride::Disabled;
				MediaPlayer->OpenSourceWithOptions(MediaSource, Options);

				continue;
			}

			const FTimespan MediaDuration = MediaPlayer->GetDuration();
			if (!MediaDuration.IsZero() && StartTime < MediaDuration)
			{
				if (FMath::Abs(MediaPlayer->GetTime() - StartTime) >= TwoFrames)
				{
					MediaPlayer->Seek(StartTime);
				}
				MediaPlayer->SetRate(0.0f);
			}
		}
	}

private:

	UMediaSource* MediaSource;
	FTimespan StartTime;
	bool bLoop;
};


struct FMediaPlayerSectionExecutionToken : IMovieSceneExecutionToken
{
	FMediaPlayerSectionExecutionToken(UMediaSource* InMediaSource, FTimespan InCurrentTime, bool bInLoop)
		: MediaSource(InMediaSource)
		, CurrentTime(InCurrentTime)
		, bLoop(bInLoop)
	{ }

	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		using namespace PropertyTemplate;
		const FSectionData& SectionData = PersistentData.GetSectionData<FSectionData>();

		const FTimespan TwoFrames = FTimespan::FromSeconds(1 / 12.f);

		for (TWeakObjectPtr<> WeakObject : Player.FindBoundObjects(Operand))
		{
			UObject* BoundObject = WeakObject.Get();
			if (!BoundObject)
			{
				continue;
			}

			UMediaPlayer* MediaPlayer = SectionData.PropertyBindings->GetCurrentValue<UMediaPlayer*>(*BoundObject);
			if (!MediaPlayer)
			{
				continue;
			}

			{
				FScopedPreAnimatedCaptureSource CaptureSource(&Player.PreAnimatedState, PersistentData.GetSectionKey(), true);
				Player.PreAnimatedState.SavePreAnimatedState(*MediaPlayer, TMovieSceneAnimTypeID<FMediaPlayerSectionExecutionToken>(), TStatelessPreAnimatedTokenProducer<FMediaPlayerPreAnimatedState>());
			}

			// Ensure the media player is playing this media
			const int32 PlaylistIndex = MediaPlayer->GetPlaylistIndex();
			if (PlaylistIndex == INDEX_NONE || !MediaPlayer->GetPlaylist() || MediaPlayer->GetPlaylist()->Get(PlaylistIndex) != MediaSource)
			{
				FMediaPlayerOptions Options;
				Options.SeekTime   = CurrentTime;
				Options.PlayOnOpen = Context.GetStatus() == EMovieScenePlayerStatus::Playing ? EMediaPlayerOptionBooleanOverride::Enabled : EMediaPlayerOptionBooleanOverride::Disabled;
				Options.Loop       = bLoop ? EMediaPlayerOptionBooleanOverride::Enabled : EMediaPlayerOptionBooleanOverride::Disabled;
				MediaPlayer->OpenSourceWithOptions(MediaSource, Options);
			}

			const FTimespan MediaDuration = MediaPlayer->GetDuration();
			if (MediaDuration.IsZero())
			{
				continue;
			}


			if (bLoop)
			{
				CurrentTime = CurrentTime % MediaDuration;
			}
			else if (CurrentTime > MediaDuration)
			{
				MediaPlayer->Pause();
				continue;
			}

			const FTimespan MediaTime = MediaPlayer->GetTime();

			if (Context.GetStatus() == EMovieScenePlayerStatus::Playing)
			{
				if (!MediaPlayer->IsPlaying())
				{
					if (FMath::Abs(MediaTime - CurrentTime) >= TwoFrames)
					{
						MediaPlayer->Seek(CurrentTime);
					}

					if (MediaPlayer->GetRate() != 1.0f)
					{
						MediaPlayer->SetRate(1.0f);
					}
				}
				else if (Context.HasJumped())
				{
					if (FMath::Abs(MediaTime - CurrentTime) >= TwoFrames)
					{
						MediaPlayer->Seek(CurrentTime);
					}
				}
			}
			else
			{
				if (FMath::Abs(MediaTime - CurrentTime) >= TwoFrames) 
				{
					MediaPlayer->Seek(CurrentTime);
				}

				// Seek can sometimes cause playback to start - ensure that doesn't happen
				MediaPlayer->Pause();
			}
		}
	}

private:

	UMediaSource* MediaSource;
	FTimespan CurrentTime;
	bool bLoop;
};


FMovieSceneMediaPlayerPropertySectionTemplate::FMovieSceneMediaPlayerPropertySectionTemplate(const UMovieSceneMediaPlayerPropertySection* InSection, const UMovieSceneMediaPlayerPropertyTrack* InTrack)
	: FMovieScenePropertySectionTemplate(InTrack->GetPropertyName(), InTrack->GetPropertyPath().ToString())
{
	MediaSource = InSection->MediaSource;
	bLoop = InSection->bLoop;

	if (InSection->HasStartFrame())
	{
		SectionStartFrame = InSection->GetRange().GetLowerBoundValue();
	}
}

void FMovieSceneMediaPlayerPropertySectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	// Can't do anything without a media source
	if (!MediaSource)
	{
		return;
	}

	// @todo: account for video time dilation if/when these are added
	if (Context.IsPreRoll())
	{
		const FFrameRate   FrameRate  = Context.GetFrameRate();
		const FFrameNumber StartFrame = Context.HasPreRollEndTime() ? Context.GetPreRollEndFrame() - SectionStartFrame : 0;

		const int64 DenominatorTicks = FrameRate.Denominator * ETimespan::TicksPerSecond;
		const int64 StartTicks = FMath::DivideAndRoundNearest(int64(StartFrame.Value * DenominatorTicks), int64(FrameRate.Numerator));

		ExecutionTokens.Add(FMediaPlayerSectionPreRollExecutionToken(MediaSource, FTimespan(StartTicks), bLoop));
	}
	else
	{
		const FFrameRate FrameRate = Context.GetFrameRate();
		const FFrameTime FrameTime(Context.GetTime().FrameNumber - SectionStartFrame);
		const int64 DenominatorTicks = FrameRate.Denominator * ETimespan::TicksPerSecond;
		const int64 FrameTicks = FMath::DivideAndRoundNearest(int64(FrameTime.GetFrame().Value * DenominatorTicks), int64(FrameRate.Numerator));
		const int64 FrameSubTicks = FMath::DivideAndRoundNearest(int64(FrameTime.GetSubFrame() * DenominatorTicks), int64(FrameRate.Numerator));

		ExecutionTokens.Add(FMediaPlayerSectionExecutionToken(MediaSource, FTimespan(FrameTicks + FrameSubTicks), bLoop));
	}
}
