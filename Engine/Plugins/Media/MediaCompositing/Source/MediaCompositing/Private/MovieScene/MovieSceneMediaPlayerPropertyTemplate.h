// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieScenePropertyTemplate.h"

#include "MovieSceneMediaPlayerPropertyTemplate.generated.h"

class UMediaSource;
class UMovieSceneMediaPlayerPropertyTrack;
class UMovieSceneMediaPlayerPropertySection;


USTRUCT()
struct FMovieSceneMediaPlayerPropertySectionTemplate
	: public FMovieScenePropertySectionTemplate
{
	GENERATED_BODY()

	/** Default constructor. */
	FMovieSceneMediaPlayerPropertySectionTemplate() : MediaSource(nullptr), bLoop(false) { }
	FMovieSceneMediaPlayerPropertySectionTemplate(const UMovieSceneMediaPlayerPropertySection* InSection, const UMovieSceneMediaPlayerPropertyTrack* InTrack);

public:

	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void SetupOverrides() override { EnableOverrides(RequiresSetupFlag); }

private:

	UPROPERTY()
	TObjectPtr<UMediaSource> MediaSource;

	UPROPERTY()
	FFrameNumber SectionStartFrame;

	UPROPERTY()
	bool bLoop;
};
