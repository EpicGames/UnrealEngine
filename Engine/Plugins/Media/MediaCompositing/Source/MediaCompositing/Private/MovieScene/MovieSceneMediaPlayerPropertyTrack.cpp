// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneMediaPlayerPropertyTrack.h"

#include "MediaSource.h"
#include "MovieScene.h"
#include "MovieSceneMediaPlayerPropertySection.h"
#include "MovieSceneMediaPlayerPropertyTemplate.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMediaPlayerPropertyTrack)


UMovieSceneMediaPlayerPropertyTrack::UMovieSceneMediaPlayerPropertyTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	EvalOptions.bCanEvaluateNearestSection = false;
	EvalOptions.bEvalNearestSection = false;
	EvalOptions.bEvaluateInPreroll = true;
	EvalOptions.bEvaluateInPostroll = true;

#if WITH_EDITORONLY_DATA
	TrackTint = FColor(0, 0, 0, 200);
#endif
}


bool UMovieSceneMediaPlayerPropertyTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneMediaPlayerPropertySection::StaticClass();
}

UMovieSceneSection* UMovieSceneMediaPlayerPropertyTrack::CreateNewSection()
{
	return NewObject<UMovieSceneMediaPlayerPropertySection>(this, NAME_None, RF_Transactional);
}

FMovieSceneEvalTemplatePtr UMovieSceneMediaPlayerPropertyTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneMediaPlayerPropertySectionTemplate(CastChecked<const UMovieSceneMediaPlayerPropertySection>(&InSection), this);
}

