// Copyright Epic Games, Inc. All Rights Reserved.


#include "MovieSceneMediaPlayerStore.h"

#include "MediaPlayer.h"
#include "Misc/CoreDelegates.h"

namespace UE::MediaCompositing::Private
{
	static FSharedPersistentDataKey MediaPlayerDataContainerSharedKey(FMovieSceneSharedDataId::Allocate(), FMovieSceneEvaluationOperand());
}

FMovieSceneMediaPlayerStore::FMovieSceneMediaPlayerStore()
{
	OnEndFrameDelegateHandle = FCoreDelegates::OnEndFrame.AddRaw(this, &FMovieSceneMediaPlayerStore::OnEndFrame);
	OnPreExitDelegateHandle = FCoreDelegates::OnPreExit.AddRaw(this, &FMovieSceneMediaPlayerStore::OnPreExit);
}

FMovieSceneMediaPlayerStore::~FMovieSceneMediaPlayerStore()
{
	FCoreDelegates::OnEndFrame.Remove(OnEndFrameDelegateHandle);
	FCoreDelegates::OnPreExit.Remove(OnPreExitDelegateHandle);
	CloseRemainingPlayers();
}

void FMovieSceneMediaPlayerStore::ScheduleMediaPlayerForRelease(const FObjectKey& InPersistentObjectKey, UMediaPlayer* InMediaPlayer)
{
	// Validation code -- There shouldn't be more than one player per section in the current design.
	if (UMediaPlayer** ExistingMediaPlayer = MediaPlayers.Find(InPersistentObjectKey))
	{
		if (IsValid(*ExistingMediaPlayer) && *ExistingMediaPlayer != InMediaPlayer)
		{
			CloseMediaPlayer(*ExistingMediaPlayer);
		}
	}
	
	MediaPlayers.Add(InPersistentObjectKey, InMediaPlayer);
}

UMediaPlayer* FMovieSceneMediaPlayerStore::TryAcquireMediaPlayer(const FObjectKey& InPersistentObjectKey)
{
	if (UMediaPlayer** FoundMediaPlayer = MediaPlayers.Find(InPersistentObjectKey))
	{
		UMediaPlayer* MediaPlayer = *FoundMediaPlayer;
		MediaPlayers.Remove(InPersistentObjectKey);
		return MediaPlayer;
	}
	return nullptr;
}

void FMovieSceneMediaPlayerStore::SetMediaPlayerPriority(const UMediaPlayer* InMediaPlayer, const FMovieSceneMediaPlayerPriority& InPriority)
{
	// Prune stale keys to avoid unbounded growth.
	for (TMap<FObjectKey, FMovieSceneMediaPlayerPriority>::TIterator It(MediaPlayerPriorities); It; ++It)
	{
		if (!It->Key.ResolveObjectPtr())
		{
			It.RemoveCurrent();
		}
	}

	MediaPlayerPriorities.Add(InMediaPlayer, InPriority);
}

TOptional<FMovieSceneMediaPlayerPriority> FMovieSceneMediaPlayerStore::GetMediaPlayerPriority(const UMediaPlayer* InMediaPlayer) const
{
	if (InMediaPlayer)
	{
		if (const FMovieSceneMediaPlayerPriority* Priority = MediaPlayerPriorities.Find(InMediaPlayer))
		{
			return *Priority;
		}
	}
	return TOptional<FMovieSceneMediaPlayerPriority>();
}

void FMovieSceneMediaPlayerStore::CloseMediaPlayer(UMediaPlayer* InMediaPlayer, bool bInCleanUpBeforeDestroy)
{
	if (IsValid(InMediaPlayer))
	{
		InMediaPlayer->Close();
		if (bInCleanUpBeforeDestroy)
		{
			InMediaPlayer->CleanUpBeforeDestroy();
		}
		InMediaPlayer->RemoveFromRoot();
	}
}

void FMovieSceneMediaPlayerStore::OnEndFrame()
{
	CloseRemainingPlayers();
}

void FMovieSceneMediaPlayerStore::OnPreExit()
{
	CloseRemainingPlayers();
}

void FMovieSceneMediaPlayerStore::CloseRemainingPlayers()
{
	for (const TPair<FObjectKey, UMediaPlayer*>& MediaPlayerEntry : MediaPlayers)
	{
		CloseMediaPlayer(MediaPlayerEntry.Value);
		
		// Clean up associated auxiliary entry for this player.
		MediaPlayerPriorities.Remove(MediaPlayerEntry.Value);
	}
	MediaPlayers.Reset();
}

FMovieSceneMediaPlayerStoreContainer::FMovieSceneMediaPlayerStoreContainer()
{
	MediaPlayerStore = MakeShared<FMovieSceneMediaPlayerStore>();
}

FMovieSceneMediaPlayerStoreContainer& FMovieSceneMediaPlayerStoreContainer::GetOrAdd(FPersistentEvaluationData& InPersistentData)
{
	return InPersistentData.GetOrAdd<FMovieSceneMediaPlayerStoreContainer>(UE::MediaCompositing::Private::MediaPlayerDataContainerSharedKey);
}
