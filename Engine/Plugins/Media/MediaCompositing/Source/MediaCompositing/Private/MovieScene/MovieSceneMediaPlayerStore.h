// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/PersistentEvaluationData.h"
#include "UObject/ObjectKey.h"

class UMediaPlayer;

/**
 * Defines a priority for media players used in media sections.
 * The priority is derived from the section's information and used for comparing
 * media players when resolving evaluation conflicts.
 */
struct FMovieSceneMediaPlayerPriority
{
	/** Primary sort priority (hierarchical) */
	int32 Priority = 0;
	/** Secondary sort temporal priority */
	FFrameNumber SectionStartFrame = FFrameNumber(0);

	bool operator==(const FMovieSceneMediaPlayerPriority& InOther) const
	{
		return Priority == InOther.Priority && SectionStartFrame == InOther.SectionStartFrame;
	}

	bool operator >(const FMovieSceneMediaPlayerPriority& InOther) const
	{
		if (Priority != InOther.Priority)
		{
			return Priority > InOther.Priority;
		}
		return SectionStartFrame > InOther.SectionStartFrame;
	}

	bool operator <(const FMovieSceneMediaPlayerPriority& InOther) const
	{
		if (Priority != InOther.Priority)
		{
			return Priority < InOther.Priority;
		}
		return SectionStartFrame < InOther.SectionStartFrame;
	}
};

/**
 * Keep a temporary store of media players so they can be reused after recompiling a media section.
 * During the track evaluation, the expired templates will be torn down first, then the new templates
 * get initialized.
 *
 * The torn down templates will submit their media player in this store associated with the owning
 * media section, so that the new template data (for the media section) can reuse the same player.
 * If players are not reused by the end of the evaluation, it means the media section is not evaluating anymore and
 * the remaining players can be closed and discarded.
 */
class FMovieSceneMediaPlayerStore
{
public:
	FMovieSceneMediaPlayerStore();
	virtual ~FMovieSceneMediaPlayerStore();

	/**
	 * Associate the given media player to the persistent object.
	 * If not acquired again with the given persistent object by the end of the frame,
	 * it will be closed and discarded.
	 * 
	 * @param InPersistentObjectKey Persistent object (ex section) we want to associate the player with.
	 * @param InMediaPlayer Media player to reuse.
	 */
	void ScheduleMediaPlayerForRelease(const FObjectKey& InPersistentObjectKey, UMediaPlayer* InMediaPlayer);

	/**
	 * @brief Get the media player registered to the given persistent object. Remove from pool if found.
	 * @param InPersistentObjectKey Persistent object (ex section) the media player is associated with.
	 * @return pointer to media player if found, null otherwise.
	 */
	UMediaPlayer* TryAcquireMediaPlayer(const FObjectKey& InPersistentObjectKey);

	/**
	 * Associate a priority to media players used to play a media section.
	 * This priority is used to resolve which player has access to a media texture in
	 * case of overlap of multiple media sections with the same media texture.
	 * 
	 * @param InMediaPlayer Media player
	 * @param InPriority Priority
	 */
	void SetMediaPlayerPriority(const UMediaPlayer* InMediaPlayer, const FMovieSceneMediaPlayerPriority& InPriority);

	/**
	 * Retrieve the associated priority for the media player (if it has one).
	 * Only players managed by media section data will have a priority associated to them.
	 * 
	 * @param InMediaPlayer Media Player 
	 * @return Associated priority, if found, for this player.
	 */
	TOptional<FMovieSceneMediaPlayerPriority> GetMediaPlayerPriority(const UMediaPlayer* InMediaPlayer) const;
	
	/**
	 * @brief Close the given media player and remove from root.
	 * @param InMediaPlayer Input media player, validity is checked.
	 * @param bInCleanUpBeforeDestroy Optionally calls CleanUpBeforeDestroy() to ensure player resources a freed immediately.
	 */
	static void CloseMediaPlayer(UMediaPlayer* InMediaPlayer, bool bInCleanUpBeforeDestroy = true);

private:
	/** Delegate handler for OnEndFrame. This is used to hook up CloseRemainingPlayers() somewhere after the end of the sequence evaluation. */
	void OnEndFrame();

	/** Delegate handler for OnPreExit. This is used to hook up CloseRemainingPlayers() before the final GC in case the application exits. */
	void OnPreExit();

	/** Called after the evaluation is completed to close all non-acquired players. */
	void CloseRemainingPlayers();

	/** Map of media players. */
	TMap<FObjectKey, UMediaPlayer*> MediaPlayers;

	/** Map of media player priorities used to resolve media texture access. */
	TMap<FObjectKey, FMovieSceneMediaPlayerPriority> MediaPlayerPriorities;

	FDelegateHandle OnEndFrameDelegateHandle;
	FDelegateHandle OnPreExitDelegateHandle;
};

/**
 * Persistent Evaluation Data Wrapper for the media player association store.
 * Using this pattern to be able to get a shared/weak ptr to the media player store so the media section data,
 * which is also a persistent evaluation data, can keep a weak ptr to solving the order of destruction issue.
 */
class FMovieSceneMediaPlayerStoreContainer : public IPersistentEvaluationData
{
public:
	FMovieSceneMediaPlayerStoreContainer();
	virtual ~FMovieSceneMediaPlayerStoreContainer() override = default;

	/**
	 * Get the media player data container from the persistent data. Will be created if missing.
	 */
	static FMovieSceneMediaPlayerStoreContainer& GetOrAdd(FPersistentEvaluationData& InPersistentData);

	/** Get the shared media player store. */
	TSharedPtr<FMovieSceneMediaPlayerStore> GetMediaPlayerStore() const
	{
		return MediaPlayerStore;
	}

private:
	TSharedPtr<FMovieSceneMediaPlayerStore> MediaPlayerStore;
};
