// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneMediaPlayerUtils.h"

#include "MediaPlayer.h"
#include "MovieSceneMediaData.h"

namespace UE::MovieSceneMediaPlayerUtils
{
	TRange<FTimespan> AdjustPlaybackTimeRange(const TRange<FTimespan>& InSectionPlaybackRange, UMediaPlayer* InMediaPlayer, const FTimespan& InFrameDuration)
	{
		if (!InMediaPlayer->SupportsPlaybackTimeRange() || InSectionPlaybackRange.IsEmpty())
		{
			return InSectionPlaybackRange;
		}

		const TRange<FTimespan> FullPlaybackRange = InMediaPlayer->GetPlaybackTimeRange(EMediaTimeRangeType::Absolute);

		// Expand half a sequencer frame on each side to give some tolerance.
		const FTimespan RangeTolerance = InFrameDuration * 0.5f;
		TRange<FTimespan> FullPlaybackRangeWithTolerance(
			TRangeBound<FTimespan>::Inclusive(FullPlaybackRange.GetLowerBoundValue() - RangeTolerance),
			TRangeBound<FTimespan>::Inclusive(FullPlaybackRange.GetUpperBoundValue() + RangeTolerance));

		// Only apply clamping if the section range is inside (with tolerance) the player's full range.
		if (FullPlaybackRangeWithTolerance.Contains(InSectionPlaybackRange))
		{
			return TRange<FTimespan>::Intersection(InSectionPlaybackRange, FullPlaybackRange);
		}

		// Media player only supports the specification of a playback range that is within the clip duration.
		return FullPlaybackRange;
	}

	void SetPlayerPlaybackTimeRange(UMediaPlayer* InMediaPlayer, const TRange<FTimespan>& InPlaybackRange)
	{
		if (!InMediaPlayer->SupportsPlaybackTimeRange())
		{
			return;
		}
		
		const TRange<FTimespan> CurrentPlaybackRange = InMediaPlayer->GetPlaybackTimeRange(EMediaTimeRangeType::Current);

		if (!InPlaybackRange.IsEmpty())
		{
			if (CurrentPlaybackRange != InPlaybackRange)
			{
				InMediaPlayer->SetPlaybackTimeRange(InPlaybackRange);
			}
		}
		else
		{
			const TRange<FTimespan> FullPlaybackRange = InMediaPlayer->GetPlaybackTimeRange(EMediaTimeRangeType::Absolute);

			// Restore playback range to full clip length (default)
			if (CurrentPlaybackRange != FullPlaybackRange)
			{
				InMediaPlayer->SetPlaybackTimeRange(FullPlaybackRange);
			}
		}
	}

	FTimespan ClampTimeToPlaybackRange(const FTimespan& InMediaTime, UMediaPlayer* InMediaPlayer, const FMovieSceneMediaPlaybackParams& InPlaybackParams)
	{
		const FTimespan Duration = InMediaPlayer->GetDuration();
		if (Duration == FTimespan::Zero())
		{
			return InMediaTime;
		}

		FTimespan MediaTime = InMediaTime;

		if (InPlaybackParams.bIsLooping)
		{
			MediaTime = MediaTime % Duration;
		}

		const FTimespan ClampTolerance = InPlaybackParams.FrameDuration * 0.5f;
		
		if (!InPlaybackParams.SectionTimeRange.IsEmpty() && InMediaPlayer->SupportsPlaybackTimeRange())
		{
			const TRange<FTimespan> AdjustedRange = AdjustPlaybackTimeRange(InPlaybackParams.SectionTimeRange, InMediaPlayer, InPlaybackParams.FrameDuration);
			MediaTime = FMath::Clamp(MediaTime, AdjustedRange.GetLowerBoundValue(), AdjustedRange.GetUpperBoundValue() - ClampTolerance);
		}
		else
		{
			MediaTime = FMath::Clamp(MediaTime, FTimespan::Zero(), Duration - ClampTolerance);
		}
		return MediaTime;
	}
}
