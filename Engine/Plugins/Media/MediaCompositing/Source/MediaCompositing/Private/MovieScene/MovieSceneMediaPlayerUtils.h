// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Timespan.h"

class UMediaPlayer;
struct FMovieSceneMediaPlaybackParams;

namespace UE::MovieSceneMediaPlayerUtils
{
	/**
	 * Utility function to adjust the playback range of the section according to proper looping logic
	 * and what is possible to do with the current player's playback range.
	 *
	 * In more detail, the current player's playback range must be smaller or equal to the full clip length.
	 * This limits us to the case where the section's playback range is also smaller than the full clip. We
	 * can't specify a playback range that would be 2.5 times the full clip length for instance. In that case,
	 * another method of looping will have to be implemented (ex: using a second player).
	 *
	 * @param InSectionPlaybackRange Playback range derived from the media section
	 * @param InMediaPlayer Player to adjust against
	 * @param InFrameDuration Sequencer frame duration, used for tolerance
	 * @return 
	 */
	TRange<FTimespan> AdjustPlaybackTimeRange(const TRange<FTimespan>& InSectionPlaybackRange, UMediaPlayer* InMediaPlayer, const FTimespan& InFrameDuration);

	/**
	 * Utility function to set the specified playback time range.
	 * @param InMediaPlayer Player to use
	 * @param InPlaybackRange Specifies the playback range to set. Must be already clamped. If empty, player's full range is restored.
	 */
	void SetPlayerPlaybackTimeRange(UMediaPlayer* InMediaPlayer, const TRange<FTimespan>& InPlaybackRange);

	/**
	 * Ensures that the given time is properly clamped to player's playback range.
	 */
	FTimespan ClampTimeToPlaybackRange(const FTimespan& InMediaTime, UMediaPlayer* InMediaPlayer, const FMovieSceneMediaPlaybackParams& InPlaybackParams);
}