// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneMediaSection.h"

#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "MediaPlayerProxyInterface.h"
#include "MovieScene.h"
#include "Misc/FrameRate.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMediaSection)

#define LOCTEXT_NAMESPACE "MovieSceneMediaSection"

namespace
{
	FFrameNumber GetStartOffsetAtTrimTime(FQualifiedFrameTime TrimTime, FFrameNumber StartOffset, FFrameNumber StartFrame)
	{
		return StartOffset + TrimTime.Time.FrameNumber - StartFrame;
	}
}

/* UMovieSceneMediaSection interface
 *****************************************************************************/

UMovieSceneMediaSection::UMovieSceneMediaSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MediaSourceProxyIndex(0)
	, bLooping(true)
	, bUseExternalMediaPlayer(false)
	, TextureIndex(0)
	, bManualFrameRateAlignment(false)
	, FrameRateAlignment(24, 1)
	, bHasMediaPlayerProxy(false)
{
#if WITH_EDITORONLY_DATA
	ThumbnailReferenceOffset = 0.f;
#endif

	EvalOptions.CompletionMode = EMovieSceneCompletionMode::RestoreState;
	BlendType = EMovieSceneBlendType::Absolute;
	ChannelCanPlayerBeOpen.SetDefault(true);

#if WITH_EDITOR
	static const FMovieSceneChannelMetaData MetaData("CanPlayerBeOpen", LOCTEXT("CanPlayerBeOpenChannelText", "Can the media player be open."));
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(ChannelCanPlayerBeOpen, MetaData, TMovieSceneExternalValue<bool>::Make());
#else
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(ChannelCanPlayerBeOpen);
#endif
}

void UMovieSceneMediaSection::PostInitProperties()
{
	Super::PostInitProperties();

	UMovieScene* Outer = GetTypedOuter<UMovieScene>();
	FFrameRate TickResolution = Outer ? Outer->GetTickResolution() : FFrameRate(24, 1);

	// media tracks have some preroll by default to precache frames
	SetPreRollFrames( (0.5 * TickResolution).RoundToFrame().Value );
}

void UMovieSceneMediaSection::TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys)
{
	if (TryModify())
	{
		if (bTrimLeft)
		{
			StartFrameOffset = HasStartFrame() ? GetStartOffsetAtTrimTime(TrimTime, StartFrameOffset, GetInclusiveStartFrame()) : 0;
		}

		Super::TrimSection(TrimTime, bTrimLeft, bDeleteKeys);
	}
}

UMovieSceneSection* UMovieSceneMediaSection::SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys)
{
	const FFrameNumber InitialStartFrameOffset = StartFrameOffset;

	const FFrameNumber NewOffset = HasStartFrame() ? GetStartOffsetAtTrimTime(SplitTime, StartFrameOffset, GetInclusiveStartFrame()) : 0;

	UMovieSceneSection* NewSection = Super::SplitSection(SplitTime, bDeleteKeys);
	if (NewSection != nullptr)
	{
		UMovieSceneMediaSection* NewMediaSection = Cast<UMovieSceneMediaSection>(NewSection);
		NewMediaSection->StartFrameOffset = NewOffset;
	}

	// Restore original offset modified by splitting
	StartFrameOffset = InitialStartFrameOffset;

	return NewSection;
}

TOptional<FFrameTime> UMovieSceneMediaSection::GetOffsetTime() const
{
	return TOptional<FFrameTime>(StartFrameOffset);
}

void UMovieSceneMediaSection::MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate)
{
	if (StartFrameOffset.Value > 0)
	{
		FFrameNumber NewStartFrameOffset = ConvertFrameTime(FFrameTime(StartFrameOffset), SourceRate, DestinationRate).FloorToFrame();
		StartFrameOffset = NewStartFrameOffset;
	}
}

void UMovieSceneMediaSection::OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	UE::MovieScene::FFixedObjectBindingID FixedBindingID = MediaSourceProxyBindingID.ResolveToFixed(LocalSequenceID, SharedPlaybackState);

	if (OldFixedToNewFixedMap.Contains(FixedBindingID))
	{
		Modify();

		const FMovieSceneSequenceHierarchy* Hierarchy = SharedPlaybackState->GetHierarchy();
		MediaSourceProxyBindingID = OldFixedToNewFixedMap[FixedBindingID].ConvertToRelative(LocalSequenceID, Hierarchy);
	}
}

void UMovieSceneMediaSection::GetReferencedBindings(TArray<FGuid>& OutBindings)
{
	OutBindings.Add(MediaSourceProxyBindingID.GetGuid());
}

UObject* UMovieSceneMediaSection::GetSourceObject() const
{
	return GetMediaSource();
}

UMediaSource* UMovieSceneMediaSection::GetMediaSource() const
{
	return MediaSource.Get();
}

UMediaSource* UMovieSceneMediaSection::GetMediaSourceOrProxy(IMovieScenePlayer& Player, FMovieSceneSequenceID SequenceID) const
{
	return GetMediaSourceOrProxy(Player, SequenceID, MediaSource, MediaSourceProxyBindingID, MediaSourceProxyIndex);
}

UMediaSource* UMovieSceneMediaSection::GetMediaSourceOrProxy(IMovieScenePlayer& InPlayer, FMovieSceneSequenceID InSequenceID, UMediaSource* InMediaSource, const FMovieSceneObjectBindingID& InMediaSourceProxyBindingID, int32 InMediaSourceProxyIndex)
{
	UMediaSource* OutMediaSource = InMediaSource;

	for (TWeakObjectPtr<> WeakObject : InMediaSourceProxyBindingID.ResolveBoundObjects(InSequenceID, InPlayer))
	{
		if (UObject* Object = WeakObject.Get())
		{
			if (AActor* Actor = Cast<AActor>(Object))
			{
				// Loop over all the components and see if any are a proxy.
				TArray<UActorComponent*> ActorComponents;
				Actor->GetComponents(ActorComponents);
				for (UActorComponent* Component : ActorComponents)
				{
					IMediaPlayerProxyInterface* Proxy = Cast<IMediaPlayerProxyInterface>(Component);
					if (Proxy != nullptr)
					{
						OutMediaSource = Proxy->ProxyGetMediaSourceFromIndex(InMediaSourceProxyIndex);
						break;
					}
				}
				if (OutMediaSource != nullptr)
				{
					break;
				}
			}
			else if (IMediaPlayerProxyInterface* Proxy = Cast<IMediaPlayerProxyInterface>(Object))
			{
				OutMediaSource = Proxy->ProxyGetMediaSourceFromIndex(InMediaSourceProxyIndex);
				break;
			}
		}
	}

	return OutMediaSource;
}

#undef LOCTEXT_NAMESPACE

