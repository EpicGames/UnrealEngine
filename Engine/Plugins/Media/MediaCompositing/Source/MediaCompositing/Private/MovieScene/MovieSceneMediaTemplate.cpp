// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneMediaTemplate.h"

#include "IMediaAssetsModule.h"
#include "IMediaCache.h"
#include "Math/UnrealMathUtility.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "MediaPlayerProxyInterface.h"
#include "MediaSoundComponent.h"
#include "MediaSource.h"
#include "MediaSourceOptions.h"
#include "MediaTexture.h"
#include "MovieScene.h"

#include "MovieSceneMediaData.h"
#include "MovieSceneMediaPlayerStore.h"
#include "MovieSceneMediaPlayerUtils.h"
#include "MovieSceneMediaSection.h"
#include "MovieSceneMediaTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMediaTemplate)


#define MOVIESCENEMEDIATEMPLATE_TRACE_EVALUATION 0

static TAutoConsoleVariable<bool> CVarUpdatePlaybackRange(
	TEXT("MediaTrack.UpdatePlaybackRange"),
	true,
	TEXT("Update Player PlaybackRange. Improves looping performance with better pre-caching."),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarAvoidSeekOnJump(
	TEXT("MediaTrack.AvoidSeekOnJump"),
	true,
	TEXT("When the play head jumps (on loop for instance), seek only if needed, i.e. if player doesn't have cached samples already."),
	ECVF_Default);


/* Local helpers
 *****************************************************************************/

struct FMediaSectionBaseMediaSourceOptions
{
	FMediaSourceCacheSettings CacheSettings;
	bool bSynchronousScrubbing = false;
};

namespace UE::MediaSectionBaseMediaUtils
{
	FMediaSourceCacheSettings GetCurrentCacheSettingsFromPlayer(const UMediaPlayer* InPlayer)
	{
		FMediaSourceCacheSettings cs;
		if (InPlayer)
		{
			TOptional<FMediaPlayerOptions> ActivePlayerOptions = InPlayer->GetPlayerFacade()->ActivePlayerOptions;
			if (ActivePlayerOptions.IsSet() 
				&& ActivePlayerOptions.GetValue().InternalCustomOptions.Contains(MediaPlayerOptionValues::ImgMediaSmartCacheEnabled())
				&& ActivePlayerOptions.GetValue().InternalCustomOptions.Contains(MediaPlayerOptionValues::ImgMediaSmartCacheTimeToLookAhead()))
			{
				const FVariant vEnabled = ActivePlayerOptions.GetValue().InternalCustomOptions[MediaPlayerOptionValues::ImgMediaSmartCacheEnabled()];
				const FVariant vTTLA = ActivePlayerOptions.GetValue().InternalCustomOptions[MediaPlayerOptionValues::ImgMediaSmartCacheTimeToLookAhead()];
				if (vEnabled.GetType() == EVariantTypes::Bool && (vTTLA.GetType() == EVariantTypes::Float || vTTLA.GetType() == EVariantTypes::Double))
				{
					cs.bOverride = vEnabled.GetValue<bool>();
					cs.TimeToLookAhead = vTTLA.GetType() == EVariantTypes::Float ? (double)vTTLA.GetValue<float>() : vTTLA.GetValue<double>();
				}
			}
		}
		return cs;
	}

	/**
	 * Check if we have a sample available for the desired time, either in the sample queue or already enqueued in video sink.
	 */
	bool IsVideoSampleAvailableForTime(const TSharedRef<FMediaPlayerFacade>& InMediaPlayerFacade, const FTimespan& InTime)
	{
		// Check LastVideoSampleProcessedTimeRange to see if the sample has already been consumed.
		const TRange<FMediaTimeStamp> LastVideoSampleTimeRange = InMediaPlayerFacade->GetLastProcessedVideoSampleTimeRange();
		if (!LastVideoSampleTimeRange.IsEmpty())
		{
			// Remark: ignoring sequence and loop indices for now.
			const TRange<FTimespan> TimeRange(LastVideoSampleTimeRange.GetLowerBoundValue().Time, LastVideoSampleTimeRange.GetUpperBoundValue().Time);
			if (TimeRange.Contains(InTime))
			{
				return true;
			}
		}

		// Query the cache state.
		TRangeSet<FTimespan> CacheRangeSet;
		InMediaPlayerFacade->QueryCacheState(EMediaTrackType::Video, EMediaCacheState::Loaded, CacheRangeSet);
		return CacheRangeSet.Contains(InTime);
	}

	/**
	 * Checks if the player needs to seek to specified time.
	 */
	bool ShouldSeekForTime(const UMediaPlayer* InMediaPlayer, const FTimespan& InTime)
	{
		const TSharedRef<FMediaPlayerFacade> MediaPlayerFacade = InMediaPlayer->GetPlayerFacade();

		// Check if already seeking to desired target.
		const FMediaTimeStamp SeekTarget = MediaPlayerFacade->GetSeekTarget();
		if (SeekTarget.IsValid() && SeekTarget.Time == InTime)
		{
			return false;	// Don't need to seek if already seeking to that specified time.
		}

		// Otherwise, check if we have a sample available for the desired time, either in the sample queue or already processed.
		return !IsVideoSampleAvailableForTime(MediaPlayerFacade, InTime);
	}

	/**
	 * Utility function to obtain the frame rate used for alignment of the "block on range".
	 * @param InContext Evaluation context, which should contain a "tick" rate we can fall back to.
	 * @param InMediaSection Media section should give us the parent movie scene to obtain actual display rate.
	 * @param InPersistentData Persistent evaluation data used to retrieve the media section data.
	 * @return movie scene display rate if available, sequencer tick rate otherwise.
	 */
	FFrameRate GetAlignedFrameRate(const FMovieSceneContext& InContext, const UMovieSceneMediaSection* InMediaSection, const FPersistentEvaluationData& InPersistentData)
	{
		// Manual frame alignment has been specified.
		if (InMediaSection && InMediaSection->bManualFrameRateAlignment)
		{
			return InMediaSection->FrameRateAlignment;
		}

		// Try to get the media's frame rate.
		if (FMovieSceneMediaData* SectionData = InPersistentData.FindSectionData<FMovieSceneMediaData>())
		{
			if (UMediaPlayer* MediaPlayer = SectionData->GetMediaPlayer())
			{
				const int32 SelectedTrack = MediaPlayer->GetSelectedTrack(EMediaPlayerTrack::Video);
				const int32 SelectedFormat = MediaPlayer->GetTrackFormat(EMediaPlayerTrack::Video, SelectedTrack);
				const float Framerate = MediaPlayer->GetVideoTrackFrameRate(SelectedTrack, SelectedFormat);
				if (Framerate > 0.0f)
				{
					// Fractional frame rates are considered "drop frame"
					if (FMath::Fractional(Framerate) > UE_KINDA_SMALL_NUMBER)
					{
						return FFrameRate(FMath::CeilToInt(Framerate) * 1000, 1001);
					}
					return FFrameRate(Framerate, 1);
				}
			}
		}

		// As a fallback, we use the tick frame rate, which may also be the display rate when sequencer is running in locked mode.
		return InContext.GetFrameRate();
	}

	/**
	 * Utility to convert a frame number to a timespan at the given frame rate.
	 * The timespan is either the first or second tick of the frame.
	 * This can be used to specify the player's block on range accurately.
	 */
	FTimespan FrameNumberToTimespan(const FFrameNumber& InFrameNumber, const FFrameRate& InFrameRate)
	{
		if (InFrameRate.Numerator != 0)
		{
			// Caution: A larger denominator limits the frame number range. 
			// Example: for 59.94, 29.97 or 23.975, because denominator is 1001, the maximum frame number is going to be 1073741823 (which is ~207 days at 59.94).
			const int64 DenominatorTicks = InFrameRate.Denominator * ETimespan::TicksPerSecond;
			// Using Ceil to ensure the tick value is part of the desired frame, and not the frame before
			// regardless of the internal media player's implementation of "time to frame".
			return FTimespan(FMath::DivideAndRoundUp(int64(InFrameNumber.Value * DenominatorTicks), int64(InFrameRate.Numerator)));
		}
		return FTimespan();
	}

	/**
	 * Utility function to calculate the exact frame duration for the given frame, so that
	 * FrameTick + FrameDurationTick = exactly the last tick of the frame. 
	 * Calculating this way takes into account the different rounding given the frame rate.
	 * Some frame rates, like 30 fps have an oscillating frame duration when rounded in ticks.
	 * 
	 * @param InFrameNumber Frame number we want to calculate the exact duration of.
	 * @param InFrameRate Frame rate.
	 * @return An exact frame duration in ticks.
	 */
	FTimespan GetFrameDuration(const FFrameNumber& InFrameNumber, const FFrameRate& InFrameRate)
	{
		const FTimespan FrameStartTime = FrameNumberToTimespan(InFrameNumber, InFrameRate);
		const FTimespan NextFrameStartTime = FrameNumberToTimespan(InFrameNumber+ 1, InFrameRate);
		return FTimespan(FMath::Max(NextFrameStartTime.GetTicks() - FrameStartTime.GetTicks() - 1, 1));
	}

	/**
	 * Returns the frame duration rounded to nearest tick. So, it may have +- 1 tick of inaccuracy. 
	 * Note: for Block on range specification, the frame accurate version, GetFrameDuration,
	 * should be used instead.
	 * @param InFrameRate Input frame rate 
	 * @return rounded frame duration in ticks.
	 */
	FTimespan GetRoundedFrameDuration(const FFrameRate& InFrameRate)
	{
		return InFrameRate.Numerator != 0 ? FTimespan::FromSeconds(InFrameRate.AsInterval()) : FTimespan(1);
	}

	/**
	 * Helper function to get the aligned frame duration given the context.
	 * Same note applies as for GetRoundedFrameDuration.
	 * 
	 * @param InContext Evaluation context
	 * @param InMediaSection Media section to get display rate from.
	 * @param InPersistentData Persistent evaluation data used to retrieve the media section data.
	 * @return Rounded frame duration.
	 */
	FTimespan GetRoundedAlignedFrameDuration(const FMovieSceneContext& InContext, const UMovieSceneMediaSection* InMediaSection, const FPersistentEvaluationData& InPersistentData)
	{
		return GetRoundedFrameDuration(GetAlignedFrameRate(InContext, InMediaSection, InPersistentData));
	}

	/**
	 * Calculates the frame number aligned to the given alignment frame rate.
	 * @param InTickFrameNumber Current sequencer tick frame number
	 * @param InTickFrameRate Current sequencer tick framerate (24000, 60000, etc)
	 * @param InAlignmentFrameRate Desired alignment frame rate for the media (24, 60, etc)
	 * @return Calculated sequencer display frame number.
	 */
	FFrameNumber GetAlignedFrame(const FFrameNumber& InTickFrameNumber, const FFrameRate& InTickFrameRate, const FFrameRate& InAlignmentFrameRate)
	{
		if (InTickFrameRate.Numerator != 0 && InAlignmentFrameRate.Denominator != 0)
		{
			return FFrameRate::TransformTime(FFrameTime(InTickFrameNumber), InTickFrameRate, InAlignmentFrameRate).GetFrame();
		}
		return FFrameNumber(0);
	}

	/**
	 * Utility to convert a sequencer's (absolute) frame number to a player's timespan.
	 */
	FTimespan SequencerFrameToPlayerTime(const FMovieSceneMediaSectionParams& InMediaSectionParams, const FFrameRate& InSequencerFrameRate, const FFrameNumber& InSequencerFrame)
	{
		const FFrameNumber PlayerFrame = InSequencerFrame - InMediaSectionParams.SectionStartFrame + InMediaSectionParams.StartFrameOffset;
		return FrameNumberToTimespan(PlayerFrame, InSequencerFrameRate);
	}

	/**
	 * Converts the given sequencer frame number range in media player's time span range.
	 * @param InMediaSectionParams Media Section information
	 * @param InSequencerFrameRate Sequencer frame rate to convert frame number in time span
	 * @param InFrameRange Frame Number range to convert, must be all inclusive.
	 */
	TRange<FTimespan> SequencerFrameRangeToPlayerRange(const FMovieSceneMediaSectionParams& InMediaSectionParams, const FFrameRate& InSequencerFrameRate, const TRange<FFrameNumber>& InFrameRange)
	{
		const FTimespan LowerTime = SequencerFrameToPlayerTime(InMediaSectionParams, InSequencerFrameRate, InFrameRange.GetLowerBoundValue());
		const FTimespan UpperTime = SequencerFrameToPlayerTime(InMediaSectionParams, InSequencerFrameRate, InFrameRange.GetUpperBoundValue());
		return TRange<FTimespan>(TRangeBound<FTimespan>::Inclusive(LowerTime), TRangeBound<FTimespan>::Inclusive(UpperTime));
	}

	/**
	 * Converts the given frame number range in all inclusive bounds.
	 * Media Player requires ranges to be inclusive.
	 */
	TRange<FFrameNumber> ToAllInclusiveRange(const TRange<FFrameNumber>& InSequencerFrameRange)
	{
		TRangeBound<FFrameNumber> LowerBound = InSequencerFrameRange.GetLowerBound();
		TRangeBound<FFrameNumber> UpperBound = InSequencerFrameRange.GetUpperBound();
		if (LowerBound.IsExclusive())
		{
			LowerBound = TRangeBound<FFrameNumber>::Inclusive(LowerBound.GetValue() + 1);
		}
		if (UpperBound.IsExclusive())
		{
			UpperBound = TRangeBound<FFrameNumber>::Inclusive(UpperBound.GetValue() - 1);
		}
		return TRange<FFrameNumber>(LowerBound, UpperBound);
	}
	
	/**
	 * Calculates the intersection of the section's bounds and scene's playback range and convert to player's time range.
	 */
	TRange<FTimespan> CalculateSectionPlaybackTimeRange(const FMovieSceneMediaSectionParams& InMediaSectionParams, const UMovieSceneMediaSection* InMediaSection, const FFrameRate& InSequencerFrameRate)
	{
		TRange<FFrameNumber> SectionFrameRange = InMediaSection->GetTrueRange();
		
		if (const UMovieSceneTrack* Track = InMediaSection->GetTypedOuter<UMovieSceneTrack>())
		{
			if (const UMovieScene* Scene = Track->GetTypedOuter<UMovieScene>())
			{
				SectionFrameRange = TRange<FFrameNumber>::Intersection(Scene->GetPlaybackRange(), SectionFrameRange);
			}
		}

		return SequencerFrameRangeToPlayerRange(InMediaSectionParams, InSequencerFrameRate, ToAllInclusiveRange(SectionFrameRange));
	}

	/** Utility function to set the specified playback time range */
	void SetPlayerPlaybackTimeRange(UMediaPlayer* InMediaPlayer, const FMovieSceneMediaPlaybackParams& InPlaybackParams)
	{
		using namespace UE::MovieSceneMediaPlayerUtils;
		const TRange<FTimespan> AdjustedRange = AdjustPlaybackTimeRange(InPlaybackParams.SectionTimeRange, InMediaPlayer, InPlaybackParams.FrameDuration);
		UE::MovieSceneMediaPlayerUtils::SetPlayerPlaybackTimeRange(InMediaPlayer, AdjustedRange);
	}

	/**
	 * Helper function to set BlockOnTimeRange.
	 * @param InMediaPlayer	Media player to use
	 * @param InCurrentTime Frame time that will be the start of the range to block on. Important: Must be clamped to playback range.
	 * @param InPlaybackParams Section derived playback parameters
	 */
	void SetPlayerBlockOnTimeRange(UMediaPlayer* InMediaPlayer, const FTimespan& InCurrentTime, const FMovieSceneMediaPlaybackParams& InPlaybackParams)
	{
		FTimespan RangeLowerBound = InCurrentTime;
		FTimespan RangeUpperBound = InCurrentTime + InPlaybackParams.FrameDuration;

		// Player Facade currently (as of 5.6) has issues when the BlockOnTimeRange is partially outside the player's active range and especially when
		// it doesn't start at zero. The wrap around code for the boundaries has inconsistencies (that will need to be fixed).
		// As a temporary workaround for that, we will clamp the BlockOnTimeRange to the section's active range to make
		// sure to avoid triggering any of the Player Facade internal wrap around boundary code.
		if (InMediaPlayer->SupportsPlaybackTimeRange() && !InPlaybackParams.SectionTimeRange.IsEmpty())
		{
			TRange<FTimespan> ActiveRange = InPlaybackParams.SectionTimeRange;
		
			// BlockOnTimeRange is spanning the upper limit of the active range
			if (ActiveRange.Contains(RangeLowerBound) && RangeUpperBound > ActiveRange.GetUpperBoundValue())
			{
				RangeUpperBound = ActiveRange.GetUpperBoundValue() - FTimespan(1); // One tick inside the range to avoid wrap around.
				RangeLowerBound = ActiveRange.GetUpperBoundValue() - InPlaybackParams.FrameDuration;
			}
			// BlockOnTimeRange is spanning the lower limit of the active range
			else if (ActiveRange.Contains(RangeUpperBound) && RangeLowerBound < ActiveRange.GetLowerBoundValue())
			{
				RangeLowerBound = ActiveRange.GetLowerBoundValue() + FTimespan(1); // One tick inside the range to avoid wrap around.
				RangeUpperBound = ActiveRange.GetLowerBoundValue() + InPlaybackParams.FrameDuration;
			}
		}
		
		InMediaPlayer->SetBlockOnTimeRange(TRange<FTimespan>(RangeLowerBound, RangeUpperBound));
	}

	/** Prepare the playback parameters for the execution token. */
	FMovieSceneMediaPlaybackParams MakePlaybackParams(const FMovieSceneContext& InContext, const FMovieSceneMediaSectionParams& InParams, const UMovieSceneMediaSection* InMediaSection, const FTimespan& InFrameDuration)
	{
		FMovieSceneMediaPlaybackParams Params;
		Params.SectionTimeRange = CVarUpdatePlaybackRange.GetValueOnGameThread() ?
			CalculateSectionPlaybackTimeRange(InParams, InMediaSection, InContext.GetFrameRate()) : TRange<FTimespan>::Empty(); 
		Params.FrameDuration = InFrameDuration;
		Params.bIsLooping = InParams.bLooping;
		return Params;
	}

	/** Returns true if the player is currently closed. */
	bool IsPlayerClosed(const UMediaPlayer& InMediaPlayer)
	{
		// Fixme: IsClosed() returns false when the player was never opened.
		// We check the internal player name as a workaround for that (i.e. no player name means it was never opened and thus is currently closed). 
		return InMediaPlayer.GetPlayerName() == NAME_None || InMediaPlayer.IsClosed();
	}

	/**
	 * Utility function to find the player proxy interface object (that implements UMediaPlayerProxyInterface)
	 * from the sequencer binding operand. 
	 */
	UObject* FindMediaPlayerProxyInterfaceObject(const FMovieSceneEvaluationOperand& InOperand, IMovieScenePlayer& InScenePlayer)
	{
		if (InOperand.ObjectBindingID.IsValid())
		{
			if (IMediaAssetsModule* MediaAssetsModule = FModuleManager::LoadModulePtr<IMediaAssetsModule>("MediaAssets"))
			{
				// Iterate over all bound objects for this operand.
				for (TWeakObjectPtr<> WeakObject : InScenePlayer.FindBoundObjects(InOperand))
				{
					if (UObject* BoundObject = WeakObject.Get())
					{
						// Use the media asset module to resolve to a player proxy object.
						UObject* PlayerProxy = nullptr;
						MediaAssetsModule->GetPlayerFromObject(BoundObject, PlayerProxy);

						// Remark: we are not interested in the media player itself. The media section will create and manage its own player.
						// We are only interested in the player proxy interface, which is used for querying the media texture
						// and other rendering parameters.

						// If we have found a player proxy interface object, we can stop searching.
						if (PlayerProxy && PlayerProxy->Implements<UMediaPlayerProxyInterface>())
						{
							return PlayerProxy;
						}
					}
				}
			}
		}
		return nullptr;
	}
}

/** Base struct for execution tokens. */
struct FMediaSectionBaseExecutionToken
	: IMovieSceneExecutionToken
{
	FMediaSectionBaseExecutionToken(UMediaSource* InMediaSource, const FMediaSectionBaseMediaSourceOptions& InMediaSourceOptions, const FMovieSceneMediaPlaybackParams& InPlaybackParams, const FMovieSceneObjectBindingID& InMediaSourceProxy, int32 InMediaSourceProxyIndex)
		: BaseMediaSource(InMediaSource)
		, MediaSourceProxy(InMediaSourceProxy)
		, MediaSourceProxyIndex(InMediaSourceProxyIndex)
		, BaseMediaSourceOptions(InMediaSourceOptions)
		, PlaybackParams(InPlaybackParams)
	{
	}

	/**
	 * Gets the media source from either the proxy binding or the media source.
	 */
	UMediaSource* GetMediaSource(IMovieScenePlayer& Player, FMovieSceneSequenceID SequenceID) const
	{
		return UMovieSceneMediaSection::GetMediaSourceOrProxy(Player, SequenceID,
			BaseMediaSource, MediaSourceProxy, MediaSourceProxyIndex);
	}

	/**
	 * Returns the index to identify the media source we are using in the proxy.
	 */
	int32 GetMediaSourceProxyIndex() const { return MediaSourceProxyIndex; }

	/**
	 * Tests if we have a media source proxy.
	 */
	bool IsMediaSourceProxyValid() const { return MediaSourceProxy.IsValid(); }

	/**
	 * Gets the media source options
	 */
	const FMediaSectionBaseMediaSourceOptions& GetBaseMediaSourceOptions() const { return BaseMediaSourceOptions; }

	/** Get the section's playback parameters. */
	const FMovieSceneMediaPlaybackParams& GetPlaybackParams() const { return PlaybackParams; }

	/**
	 * Utility function to prepare the media player options for opening the media source.
	 */
	FMediaPlayerOptions MakeMediaPlayerOptions(const IMediaPlayerProxyInterface* InPlayerProxyInterface, FMovieSceneMediaData& InSectionData) const
	{
		FMediaPlayerOptions Options;
		Options.SetAllAsOptional();
		Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::Environment(), MediaPlayerOptionValues::Environment_Sequencer());

		if (InPlayerProxyInterface != nullptr)
		{
			// Set cache settings.
			const FMediaSourceCacheSettings& CacheSettings = InPlayerProxyInterface->GetCacheSettings();
			Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ImgMediaSmartCacheEnabled(), FVariant(CacheSettings.bOverride));
			Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ImgMediaSmartCacheTimeToLookAhead(), FVariant(CacheSettings.TimeToLookAhead));

			// Set the view texture for proper mips and tiles loading during pre-roll.
			// This is only done if we have a PlayerProxyInterface because it is the only case with associated visibility geometry (ex MediaPlate).
			if (UMediaTexture* ViewTexture = InSectionData.GetProxyMediaTexture())
			{
				Options.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ViewMediaTexture(), FVariant(FSoftObjectPath(ViewTexture).ToString()));
			}
		}
		return Options;
	}

private:
	UMediaSource* BaseMediaSource;
	FMovieSceneObjectBindingID MediaSourceProxy;
	int32 MediaSourceProxyIndex = 0;
	FMediaSectionBaseMediaSourceOptions BaseMediaSourceOptions;
	FMovieSceneMediaPlaybackParams PlaybackParams;
};

struct FMediaSectionPreRollExecutionToken
	: FMediaSectionBaseExecutionToken
{
	FMediaSectionPreRollExecutionToken(UMediaSource* InMediaSource, const FMediaSectionBaseMediaSourceOptions& InMediaSourceOptions, const FMovieSceneMediaPlaybackParams& InPlaybackParams, FMovieSceneObjectBindingID InMediaSourceProxy, int32 InMediaSourceProxyIndex, FTimespan InStartTimeSeconds)
		: FMediaSectionBaseExecutionToken(InMediaSource, InMediaSourceOptions, InPlaybackParams, InMediaSourceProxy, InMediaSourceProxyIndex)
		, StartTime(InStartTimeSeconds)
	{ }

	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		using namespace PropertyTemplate;

		FMovieSceneMediaData& SectionData = PersistentData.GetSectionData<FMovieSceneMediaData>();
		UMediaPlayer* MediaPlayer = SectionData.GetMediaPlayer();
		const IMediaPlayerProxyInterface* PlayerProxyInterface = Cast<IMediaPlayerProxyInterface>(SectionData.GetPlayerProxy());
		UMediaSource* MediaSource = GetMediaSource(Player, Operand.SequenceID);

		if (MediaPlayer == nullptr || MediaSource == nullptr)
		{
			return;
		}

		// open the media source if necessary
		if (MediaPlayer->GetUrl() != MediaSource->GetUrl())
		{
			SectionData.SeekOnOpen(StartTime);

			FMediaPlayerOptions Options = MakeMediaPlayerOptions(PlayerProxyInterface, SectionData);
			MediaPlayer->OpenSourceWithOptions(MediaSource, Options);
			return;
		}

		const bool bMoveToNewTime = Context.GetStatus() != EMovieScenePlayerStatus::Playing || (Context.GetStatus() == EMovieScenePlayerStatus::Playing && Context.HasJumped());
		if (bMoveToNewTime)
		{
			if (MediaPlayer->GetRate() != 0.0f)	// avoids Electra spamming "SetRate" when scrubbing in the preroll.
			{
				MediaPlayer->SetRate(0.0f);
			}
			MediaPlayer->Seek(StartTime);
			MediaPlayer->SetBlockOnTimeRange(TRange<FTimespan>::Empty());
		}
	}

private:

	FTimespan StartTime;
};

struct FMediaSectionPostRollExecutionToken
	: IMovieSceneExecutionToken
{
	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		FMovieSceneMediaData& SectionData = PersistentData.GetSectionData<FMovieSceneMediaData>();
		UMediaPlayer* MediaPlayer = SectionData.GetMediaPlayer();

		if (MediaPlayer == nullptr || UE::MediaSectionBaseMediaUtils::IsPlayerClosed(*MediaPlayer))
		{
			return;
		}

		if (MediaPlayer->GetRate() != 0.0f)	// avoids Electra spamming "SetRate" when scrubbing in post-roll.
		{
			MediaPlayer->SetRate(0.0f);
		}
	}
};

struct FMediaSectionExecutionToken
	: FMediaSectionBaseExecutionToken
{
	FMediaSectionExecutionToken(UMediaSource* InMediaSource, const FMediaSectionBaseMediaSourceOptions& InMediaSourceOptions, const FMovieSceneMediaPlaybackParams& InPlaybackParams, FMovieSceneObjectBindingID InMediaSourceProxy, int32 InMediaSourceProxyIndex, float InProxyTextureBlend, bool bInCanPlayerBeOpen, FTimespan InCurrentTime)
		: FMediaSectionBaseExecutionToken(InMediaSource, InMediaSourceOptions, InPlaybackParams, InMediaSourceProxy, InMediaSourceProxyIndex)
		, CurrentTime(InCurrentTime)
		, ProxyTextureBlend(InProxyTextureBlend)
		, bCanPlayerBeOpen(bInCanPlayerBeOpen)
	{ }

	void SeekPlayer(const FMovieSceneContext& InContext, UMediaPlayer* InMediaPlayer, const FTimespan& InMediaTime) const
	{
		if (InContext.GetStatus() == EMovieScenePlayerStatus::Scrubbing || InContext.GetStatus() == EMovieScenePlayerStatus::Stopped)
		{
			// Scrubbing outside the playback range is allowed, in this case, we need to reset it.
			if (!GetPlaybackParams().SectionTimeRange.IsEmpty() && !GetPlaybackParams().SectionTimeRange.Contains(InMediaTime))
			{
				UE::MovieSceneMediaPlayerUtils::SetPlayerPlaybackTimeRange(InMediaPlayer, TRange<FTimespan>::Empty());
			}

			if (!GetBaseMediaSourceOptions().bSynchronousScrubbing)
			{
				InMediaPlayer->Scrub(InMediaTime);
			}
			else
			{
				// Force seek here to avoid issues with block on range.
				InMediaPlayer->Seek(InMediaTime);
			}
		}
		else
		{
			using namespace UE::MovieSceneMediaPlayerUtils;
			const FTimespan MediaTime = ClampTimeToPlaybackRange(InMediaTime, InMediaPlayer, GetPlaybackParams());
			InMediaPlayer->Seek(MediaTime);
		}
	}

	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		FMovieSceneMediaData& SectionData = PersistentData.GetSectionData<FMovieSceneMediaData>();
		UMediaPlayer* MediaPlayer = SectionData.GetMediaPlayer();
		UObject* PlayerProxy = SectionData.GetPlayerProxy();
		UMediaSource* MediaSource = GetMediaSource(Player, Operand.SequenceID);

		if (MediaPlayer == nullptr || MediaSource == nullptr)
		{
			return;
		}

		// Do we have a player proxy?
		IMediaPlayerProxyInterface* PlayerProxyInterface = Cast<IMediaPlayerProxyInterface>(PlayerProxy);
		if (PlayerProxyInterface != nullptr)
		{
			PlayerProxyInterface->ProxySetTextureBlend(SectionData.GetProxyLayerIndex(), SectionData.GetProxyTextureIndex(), ProxyTextureBlend);
			// Can we control the player?
			if (PlayerProxyInterface->IsExternalControlAllowed() == false)
			{
				return;
			}

			if (SectionData.bIsAspectRatioSet == false)
			{
				if (PlayerProxyInterface->ProxySetAspectRatio(MediaPlayer))
				{
					SectionData.bIsAspectRatioSet = true;
				}
			}
		}

		using namespace UE::MediaSectionBaseMediaUtils;

		// Can we be open?
		if (bCanPlayerBeOpen == false)
		{
			if (!IsPlayerClosed(*MediaPlayer))
			{
				MediaPlayer->Close();
			}
			return;
		}

		// Check if the cache options have changed mid playback.
		//const FMediaSourceCacheSettings CurrentCacheSettings = GetBaseMediaSourceOptions().CacheSettings;
		const FMediaSourceCacheSettings CurrentCacheSettings = GetCurrentCacheSettingsFromPlayer(MediaPlayer);
		const bool bCacheSettingsChanged = PlayerProxyInterface ? CurrentCacheSettings != PlayerProxyInterface->GetCacheSettings() : false;
		const bool bEnableBlockingRequests = (Context.GetStatus() != EMovieScenePlayerStatus::Scrubbing && Context.GetStatus() != EMovieScenePlayerStatus::Stopped) || GetBaseMediaSourceOptions().bSynchronousScrubbing;

		// open the media source if necessary
		if (MediaPlayer->GetUrl() != MediaSource->GetUrl() || bCacheSettingsChanged)
		{
			SectionData.SeekOnOpen(CurrentTime);

			FMediaPlayerOptions Options = MakeMediaPlayerOptions(PlayerProxyInterface, SectionData);

			if (bEnableBlockingRequests)
			{
				// Setup an initial blocking range - MediaFramework will block (even through the opening process) in its next tick...
				SetPlayerBlockOnTimeRange(MediaPlayer, CurrentTime, GetPlaybackParams());
			}
			MediaPlayer->OpenSourceWithOptions(MediaSource, Options);
			return;
		}

		// seek on open if necessary
		// (usually should not be needed as the blocking on open should ensure we never see the player preparing here)
		if (MediaPlayer->IsPreparing())
		{
			SectionData.SeekOnOpen(CurrentTime);
			if (bEnableBlockingRequests)
			{
				SetPlayerBlockOnTimeRange(MediaPlayer, CurrentTime, GetPlaybackParams());
			}
			return;
		}

		const FTimespan MediaDuration = MediaPlayer->GetDuration();

		if (MediaDuration.IsZero())
		{
			return; // media has no length
		}

		//
		// update media player
		//

		SetPlayerPlaybackTimeRange(MediaPlayer, GetPlaybackParams());

		// Setup media time (used for seeks)
		FTimespan MediaTime = CurrentTime;

		#if MOVIESCENEMEDIATEMPLATE_TRACE_EVALUATION
			GLog->Logf(ELogVerbosity::Log, TEXT("Executing time %s, MediaTime %s"), *CurrentTime.ToString(TEXT("%h:%m:%s.%t")), *MediaTime.ToString(TEXT("%h:%m:%s.%t")));
		#endif

		if (Context.GetStatus() == EMovieScenePlayerStatus::Playing)
		{
			if (!MediaPlayer->IsPlaying())
			{
				// If the player has pre-rolled, it is ready and doesn't need to seek.
				if (ShouldSeekForTime(MediaPlayer, MediaTime))
				{
					SeekPlayer(Context, MediaPlayer, MediaTime);
				}

				// Set rate
				// (note that the DIRECTION is important, but the magnitude is not - as we use blocked playback, the range setup to block on will serve as external clock to the player,
				//  the direction is taken into account as hint for internal operation of the player)
				if (!MediaPlayer->SetRate((Context.GetDirection() == EPlayDirection::Forwards) ? 1.0f : -1.0f))
				{
					// Failed to set needed rate. Keep things blocked, as this means the player will still not be playing, this will
					// trigger a seek to each and every frame. A potentially very SLOW method of approximating backwards playback, but better
					// than nothing.
					// -> nothing to do
				}
			}
			else
			{
				// Avoid seek on jump (loop) if the player already has cached samples.
				if ((Context.HasJumped() || !SectionData.bHasBeenExecuted) && (!CVarAvoidSeekOnJump.GetValueOnGameThread() || ShouldSeekForTime(MediaPlayer, MediaTime)))
				{
					SeekPlayer(Context, MediaPlayer, MediaTime);
				}

				const float CurrentPlayerRate = MediaPlayer->GetRate();
				if (Context.GetDirection() == EPlayDirection::Forwards && CurrentPlayerRate < 0.0f)
				{
					if (!MediaPlayer->SetRate(1.0f))
					{
						// Failed to set needed rate. Keep things blocked, as this means the player will still be returning the old rate, we will get here repeatedly
						// and each time trigger a seek. A potentially very SLOW method of approximating backwards playback, but better
						// than nothing.
						SeekPlayer(Context, MediaPlayer, MediaTime);
					}
				}
				else if (Context.GetDirection() == EPlayDirection::Backwards && CurrentPlayerRate > 0.0f)
				{
					if (!MediaPlayer->SetRate(-1.0f))
					{
						// Failed to set needed rate. Keep things blocked, as this means the player will still be returning the old rate, we will get here repeatedly
						// and each time trigger a seek. A potentially very SLOW method of approximating backwards playback, but better
						// than nothing.
						SeekPlayer(Context, MediaPlayer, MediaTime);
					}
				}
			}
		}
		else
		{
			if (MediaPlayer->IsPlaying() && MediaPlayer->GetRate() != 0.0f)
			{
				MediaPlayer->SetRate(0.0f);
			}

			SeekPlayer(Context, MediaPlayer, MediaTime);
		}

		if (bEnableBlockingRequests)
		{
			// Set blocking range / time-range to display
			// (we always use the full current time for this, any adjustments to player timestamps are done internally)
			SetPlayerBlockOnTimeRange(MediaPlayer, CurrentTime, GetPlaybackParams());
		}
		else
		{
			// When scrubbing, seek requests are non-blocking.
			MediaPlayer->SetBlockOnTimeRange(TRange<FTimespan>::Empty());
		}
		
		// Mark the section data as having been evaluated.
		SectionData.bHasBeenExecuted = true;
	}

private:

	FTimespan CurrentTime;
	float ProxyTextureBlend;
	bool bCanPlayerBeOpen;
};


/* FMovieSceneMediaSectionTemplate structors
 *****************************************************************************/

FMovieSceneMediaSectionTemplate::FMovieSceneMediaSectionTemplate(const UMovieSceneMediaSection& InSection, const UMovieSceneMediaTrack& InTrack)
	: MediaSection(&InSection)
{
	Params.MediaSource = InSection.GetMediaSource();
	Params.MediaSourceProxy = InSection.GetMediaSourceProxy();
	Params.MediaSourceProxyIndex = InSection.MediaSourceProxyIndex;
	Params.MediaSoundComponent = InSection.MediaSoundComponent;
	Params.bLooping = InSection.bLooping;
	Params.StartFrameOffset = InSection.StartFrameOffset;
	Params.CacheSettings = InSection.CacheSettings;

	// If using an external media player link it here so we don't automatically create it later.
	Params.MediaPlayer = InSection.bUseExternalMediaPlayer ? InSection.ExternalMediaPlayer : nullptr;
	Params.MediaTexture = InSection.bUseExternalMediaPlayer ? nullptr : InSection.MediaTexture;

	if (InSection.HasStartFrame())
	{
		Params.SectionStartFrame = InSection.GetRange().GetLowerBoundValue();
	}
	if (InSection.HasEndFrame())
	{
		Params.SectionEndFrame = InSection.GetRange().GetUpperBoundValue();
	}
}


/* FMovieSceneEvalTemplate interface
 *****************************************************************************/

void FMovieSceneMediaSectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	UMediaSource* MediaSource = Params.MediaSource;
	if (((MediaSource == nullptr) && (Params.MediaSourceProxy.IsValid() == false)))
	{
		return;
	}
	
	// @todo: account for video time dilation if/when these are added

	bool bCanPlayerBeOpen = true;
	MediaSection->ChannelCanPlayerBeOpen.Evaluate(Context.GetTime(), bCanPlayerBeOpen);

	using namespace UE::MediaSectionBaseMediaUtils;
	const FFrameRate AlignedFrameRate = GetAlignedFrameRate(Context, MediaSection, PersistentData);
	
	if (Context.IsPreRoll() && bCanPlayerBeOpen)
	{
		const FFrameNumber StartTickFrame = Context.HasPreRollEndTime() ? Context.GetPreRollEndFrame() - Params.SectionStartFrame + Params.StartFrameOffset : Params.StartFrameOffset;
		const FFrameNumber StartAlignedFrame = GetAlignedFrame(StartTickFrame, Context.GetFrameRate(), AlignedFrameRate); 
		const FTimespan StartTime = FrameNumberToTimespan(StartAlignedFrame, AlignedFrameRate);
		const FTimespan AlignedFrameDuration = GetFrameDuration(StartAlignedFrame, AlignedFrameRate);

		FMediaSectionBaseMediaSourceOptions Options;
		Options.CacheSettings = Params.CacheSettings;
		Options.bSynchronousScrubbing = false; // not relevant during pre-roll
		const FMovieSceneMediaPlaybackParams PlaybackParams = MakePlaybackParams(Context, Params, MediaSection, AlignedFrameDuration);
		ExecutionTokens.Add(FMediaSectionPreRollExecutionToken(MediaSource, Options, PlaybackParams, Params.MediaSourceProxy, Params.MediaSourceProxyIndex, StartTime));
	}
	else if (Context.IsPostRoll() && bCanPlayerBeOpen)
	{
		ExecutionTokens.Add(FMediaSectionPostRollExecutionToken());
	}
	else if (!Context.IsPostRoll() && (Context.GetTime().FrameNumber < Params.SectionEndFrame))
	{
		const FFrameNumber TickFrameNumber = Context.GetTime().FrameNumber - Params.SectionStartFrame + Params.StartFrameOffset;
		const FFrameNumber AlignedFrameNumber = GetAlignedFrame(TickFrameNumber, Context.GetFrameRate(), AlignedFrameRate); 
		const FTimespan FrameTime = FrameNumberToTimespan(AlignedFrameNumber, AlignedFrameRate);
		const FTimespan AlignedFrameDuration = GetFrameDuration(AlignedFrameNumber, AlignedFrameRate);
		
		const float ProxyTextureBlend = MediaSection->EvaluateEasing(Context.GetTime());

		#if MOVIESCENEMEDIATEMPLATE_TRACE_EVALUATION
			GLog->Logf(ELogVerbosity::Log, TEXT("Evaluating (%s) frame %i+%f, FrameRate %i/%i, FrameTicks %lld, FrameDurationTicks %lld"),
				*StaticEnum<EMovieScenePlayerStatus::Type>()->GetNameStringByValue(Context.GetStatus()),
				Context.GetTime().GetFrame().Value,
				Context.GetTime().GetSubFrame(),
				FrameRate.Numerator,
				FrameRate.Denominator,
				FrameTime.GetTicks(),
				AlignedFrameDuration.GetTicks()
			);
		#endif

		const UMovieSceneMediaTrack* MediaTrack = MediaSection->GetTypedOuter<UMovieSceneMediaTrack>();

		FMediaSectionBaseMediaSourceOptions Options;
		Options.CacheSettings = Params.CacheSettings;
		Options.bSynchronousScrubbing = MediaTrack ? MediaTrack->bSynchronousScrubbing : false;
		FMovieSceneMediaPlaybackParams PlaybackParams = MakePlaybackParams(Context, Params, MediaSection, AlignedFrameDuration);

		// Scrubbing or stepping outside the playback range is allowed by the sequencer,
		// in which case we have to reset the player's playback range.
		if (Context.GetStatus() == EMovieScenePlayerStatus::Scrubbing
			|| Context.GetStatus() == EMovieScenePlayerStatus::Stopped
			|| Context.GetStatus() == EMovieScenePlayerStatus::Stepping)
		{
			if (!PlaybackParams.SectionTimeRange.IsEmpty() && !PlaybackParams.SectionTimeRange.Contains(FrameTime))
			{
				// Note: this only resets the playback time range for the current evaluation.
				// It will propagate to the player, but not the persistent section data (FMovieSceneMediaData).
				PlaybackParams.SectionTimeRange = TRange<FTimespan>::Empty();
			}
		}
		
		ExecutionTokens.Add(FMediaSectionExecutionToken(MediaSource, Options, PlaybackParams, Params.MediaSourceProxy, Params.MediaSourceProxyIndex, ProxyTextureBlend, bCanPlayerBeOpen, FrameTime));
	}
}


UScriptStruct& FMovieSceneMediaSectionTemplate::GetScriptStructImpl() const
{
	return *StaticStruct();
}


void FMovieSceneMediaSectionTemplate::Initialize(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
{
	FMovieSceneMediaData* SectionData = PersistentData.FindSectionData<FMovieSceneMediaData>();
	if (SectionData == nullptr)
	{
		int32 ProxyTextureIndex = 0;
		int32 ProxyLayerIndex = 0;
		if (MediaSection != nullptr)
		{
			ProxyTextureIndex = MediaSection->TextureIndex;
			ProxyLayerIndex = MediaSection->GetRowIndex();
		}

		using namespace UE::MediaSectionBaseMediaUtils;

		// If the media section doesn't specify an external media player, we try to find the
		// media player proxy from the bound objects.
		UObject* MediaPlayerProxy = nullptr;
		if (Params.MediaPlayer == nullptr)
		{
			MediaPlayerProxy = FindMediaPlayerProxyInterfaceObject(Operand, Player);
		}

		// Add section data.
		SectionData = &PersistentData.AddSectionData<FMovieSceneMediaData>();
		const TSharedPtr<FMovieSceneMediaPlayerStore> MediaPlayerStore = FMovieSceneMediaPlayerStoreContainer::GetOrAdd(PersistentData).GetMediaPlayerStore();
		// Prepare these values in the section data to have them set in the player when it is opened (as early as possible before the first seek).
		const FTimespan FrameDuration = GetRoundedAlignedFrameDuration(Context, MediaSection, PersistentData);
		FMovieSceneMediaPlaybackParams PlaybackParams = MakePlaybackParams(Context, Params, MediaSection, FrameDuration);

		SectionData->Setup(MediaPlayerStore, MediaSection, Params.MediaPlayer, MediaPlayerProxy, ProxyLayerIndex, ProxyTextureIndex, PlaybackParams);

		// Assign the media player priority to resolve overlapping access to a common media texture.
		if (MediaPlayerStore && SectionData->GetMediaPlayer())
		{
			FMovieSceneMediaPlayerPriority PlayerPriority;
			// The proxy layer index is the "row index" of the media section, which can be used to determine priority.
			PlayerPriority.Priority = ProxyLayerIndex;
			// For the "top-left" priority evaluation, we need the section start (leftmost boundary).
			PlayerPriority.SectionStartFrame = Params.SectionStartFrame;
			
			MediaPlayerStore->SetMediaPlayerPriority(SectionData->GetMediaPlayer(), PlayerPriority);
		}
	}

	if (!ensure(SectionData != nullptr))
	{
		return;
	}

	// If the player proxy became stale, we try to restore it.
	// This may happen if the player proxy object is a spawnable.
	if (SectionData->IsPlayerProxyStale())
	{
		// Note: if the proxy is not found, it is likely that it is not spawned yet.
		// We keep it "stale" in the media section so that we keep trying to restore it until it is spawned again.
		if (UObject* MediaPlayerProxy = UE::MediaSectionBaseMediaUtils::FindMediaPlayerProxyInterfaceObject(Operand, Player))
		{
			SectionData->UpdatePlayerProxy(MediaPlayerProxy);
		}
	}

	UMediaPlayer* MediaPlayer = SectionData->GetMediaPlayer();

	if (MediaPlayer == nullptr)
	{
		return;
	}

	const bool IsEvaluating = !(Context.IsPreRoll() || Context.IsPostRoll() || (Context.GetTime().FrameNumber >= Params.SectionEndFrame));
	SectionData->Initialize(IsEvaluating);

	if (Params.MediaSoundComponent != nullptr)
	{
		if (IsEvaluating)
		{
			#if MOVIESCENEMEDIATEMPLATE_TRACE_EVALUATION
				GLog->Logf(ELogVerbosity::Log, TEXT("Setting media player %p on media sound component %p"), MediaPlayer, Params.MediaSoundComponent.Get());
			#endif

			Params.MediaSoundComponent->SetMediaPlayer(MediaPlayer);
		}
		else if (Params.MediaSoundComponent->GetMediaPlayer() == MediaPlayer)
		{
			#if MOVIESCENEMEDIATEMPLATE_TRACE_EVALUATION
				GLog->Logf(ELogVerbosity::Log, TEXT("Resetting media player on media sound component %p"), Params.MediaSoundComponent.Get());
			#endif

			Params.MediaSoundComponent->SetMediaPlayer(nullptr);
		}
	}
	
	TSharedPtr<FMediaTextureSampleQueue> PreRollSampleQueue;

	if (IsEvaluating)
	{
		// Make sure to transfer the sample queue used for pre-roll (on the first evaluation), even if it
		// isn't used by a media texture, otherwise it will block the player by not consuming the samples.
		PreRollSampleQueue = SectionData->TransferSampleQueue();
	}

	if (Params.MediaTexture != nullptr)
	{
		if (IsEvaluating)
		{
			// Don't steal the media texture from higher priority players.
			bool bCanUsePlayer = true;

			if (const TSharedPtr<FMovieSceneMediaPlayerStore> MediaPlayerStore = FMovieSceneMediaPlayerStoreContainer::GetOrAdd(PersistentData).GetMediaPlayerStore())
			{
				// Resolving concurrent media texture access using the associated media player priorities.
				// We compare the current player, i.e. the one currently assigned to the media texture, with the new one from this section.
				TOptional<FMovieSceneMediaPlayerPriority> CurrentPriority = MediaPlayerStore->GetMediaPlayerPriority(Params.MediaTexture->GetMediaPlayer());
				TOptional<FMovieSceneMediaPlayerPriority> NewPriority = MediaPlayerStore->GetMediaPlayerPriority(MediaPlayer);

				// Current rule is that the top-left most player gets the texture.
				if (CurrentPriority.IsSet() && NewPriority.IsSet() && NewPriority.GetValue() > CurrentPriority.GetValue())
				{
					bCanUsePlayer = false;
				}
			}

			if (bCanUsePlayer)
			{
#if MOVIESCENEMEDIATEMPLATE_TRACE_EVALUATION
				GLog->Logf(ELogVerbosity::Log, TEXT("Setting media player %p on media texture %p"), MediaPlayer, Params.MediaTexture.Get());
#endif
				Params.MediaTexture->SetMediaPlayer(MediaPlayer, PreRollSampleQueue);
			}
		}
		else if (Params.MediaTexture->GetMediaPlayer() == MediaPlayer)
		{
			#if MOVIESCENEMEDIATEMPLATE_TRACE_EVALUATION
				GLog->Logf(ELogVerbosity::Log, TEXT("Resetting media player on media texture %p"), Params.MediaTexture.Get());
			#endif

			Params.MediaTexture->SetMediaPlayer(nullptr);
		}
	}

	if (!UE::MediaSectionBaseMediaUtils::IsPlayerClosed(*MediaPlayer) && MediaPlayer->IsLooping() != Params.bLooping)
	{
		MediaPlayer->SetLooping(Params.bLooping);
	}
}


void FMovieSceneMediaSectionTemplate::SetupOverrides()
{
	EnableOverrides(RequiresInitializeFlag | RequiresTearDownFlag);
}


void FMovieSceneMediaSectionTemplate::TearDown(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const
{
	FMovieSceneMediaData* SectionData = PersistentData.FindSectionData<FMovieSceneMediaData>();

	if (!ensure(SectionData != nullptr))
	{
		return;
	}

	UMediaPlayer* MediaPlayer = SectionData->GetMediaPlayer();

	if (MediaPlayer == nullptr)
	{
		return;
	}

	if ((Params.MediaSoundComponent != nullptr) && (Params.MediaSoundComponent->GetMediaPlayer() == MediaPlayer))
	{
		Params.MediaSoundComponent->SetMediaPlayer(nullptr);
	}

	if ((Params.MediaTexture != nullptr) && (Params.MediaTexture->GetMediaPlayer() == MediaPlayer))
	{
		Params.MediaTexture->SetMediaPlayer(nullptr);
	}

	UObject* PlayerProxy = SectionData->GetPlayerProxy();
	if (PlayerProxy != nullptr)
	{
		IMediaPlayerProxyInterface* PlayerProxyInterface = Cast<IMediaPlayerProxyInterface>(PlayerProxy);
		if (PlayerProxyInterface != nullptr)
		{
			PlayerProxyInterface->ProxySetTextureBlend(SectionData->GetProxyLayerIndex(), SectionData->GetProxyTextureIndex(), 0.0f);
		}
	}

	SectionData->TearDown();
}

