// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneFloatChannel.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "MovieSceneObjectBindingID.h"
#include "MediaSourceOptions.h"

#include "MovieSceneMediaTemplate.generated.h"

class UMediaPlayer;
class UMovieSceneMediaSection;
class UMovieSceneMediaTrack;
class UMediaSoundComponent;
class UMediaSource;
class UMediaTexture;


USTRUCT()
struct FMovieSceneMediaSectionParams
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMediaSoundComponent> MediaSoundComponent;

	UPROPERTY()
	TObjectPtr<UMediaSource> MediaSource;

	UPROPERTY()
	FMovieSceneObjectBindingID MediaSourceProxy;

	UPROPERTY()
	int32 MediaSourceProxyIndex;

	UPROPERTY()
	TObjectPtr<UMediaTexture> MediaTexture;

	UPROPERTY()
	TObjectPtr<UMediaPlayer> MediaPlayer;

	UPROPERTY()
	FFrameNumber SectionStartFrame;

	UPROPERTY()
	FFrameNumber SectionEndFrame;

	UPROPERTY()
	bool bLooping;
	
	UPROPERTY()
	FFrameNumber StartFrameOffset;

	UPROPERTY()
	FMovieSceneFloatChannel ProxyTextureBlend;

	UPROPERTY()
	FMediaSourceCacheSettings CacheSettings;

	FMovieSceneMediaSectionParams()
		: MediaSoundComponent(nullptr)
		, MediaSource(nullptr)
		, MediaSourceProxyIndex(0)
		, MediaTexture(nullptr)
		, MediaPlayer(nullptr)
		, bLooping(false)
	{}
};


USTRUCT()
struct FMovieSceneMediaSectionTemplate
	: public FMovieSceneEvalTemplate
{
	GENERATED_BODY()

	/** Default constructor. */
	FMovieSceneMediaSectionTemplate() { }

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InSection
	 * @param InTrack
	 */
	FMovieSceneMediaSectionTemplate(const UMovieSceneMediaSection& InSection, const UMovieSceneMediaTrack& InTrack);

public:

	//~ FMovieSceneEvalTemplate interface

	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;
	virtual UScriptStruct& GetScriptStructImpl() const override;
	virtual void Initialize(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const override;
	virtual void SetupOverrides() override;
	virtual void TearDown(FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) const override;

private:

	UPROPERTY()
	FMovieSceneMediaSectionParams Params;

	UPROPERTY()
	TObjectPtr<const UMovieSceneMediaSection> MediaSection;
};
