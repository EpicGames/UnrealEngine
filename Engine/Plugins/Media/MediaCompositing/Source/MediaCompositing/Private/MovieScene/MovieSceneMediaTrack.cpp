// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneMediaTrack.h"

#include "IMediaClock.h"
#include "IMediaClockSink.h"
#include "IMediaModule.h"
#include "MediaPlayer.h"
#include "MediaPlaylist.h"
#include "MediaSource.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneMediaSection.h"
#include "MovieSceneMediaTemplate.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMediaTrack)


#define LOCTEXT_NAMESPACE "MovieSceneMediaTrack"

#if WITH_EDITOR

/**
 * Media clock sink for media track.
 */
class FMovieSceneMediaTrackClockSink
	: public IMediaClockSink
{
public:

	FMovieSceneMediaTrackClockSink(UMovieSceneMediaTrack* InOwner)
		: Owner(InOwner)
	{ }

	virtual ~FMovieSceneMediaTrackClockSink() { }

	virtual void TickOutput(FTimespan DeltaTime, FTimespan Timecode) override
	{
		if (UMovieSceneMediaTrack* OwnerPtr = Owner.Get())
		{
			Owner->TickOutput();
		}
	}

	/**
	 * Call this when the owner is destroyed.
	 */
	void OwnerDestroyed()
	{
		Owner.Reset();
	}

private:

	TWeakObjectPtr<UMovieSceneMediaTrack> Owner;
};

#endif // WITH_EDITOR

/* UMovieSceneMediaTrack interface
 *****************************************************************************/

UMovieSceneMediaTrack::UMovieSceneMediaTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, bGetDurationDelay(false)
#endif // WITH_EDITORONLY_DATA
{
	EvalOptions.bCanEvaluateNearestSection = false;
	EvalOptions.bEvalNearestSection = false;
	EvalOptions.bEvaluateInPreroll = true;
	EvalOptions.bEvaluateInPostroll = true;

	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);

#if WITH_EDITORONLY_DATA
	TrackTint = FColor(0, 0, 0, 200);
	bSupportsDefaultSections = false;
#endif
}


/* UMovieSceneMediaTrack interface
 *****************************************************************************/

UMovieSceneSection* UMovieSceneMediaTrack::AddNewMediaSourceOnRow(UMediaSource& MediaSource, FFrameNumber Time, int32 RowIndex)
{
	UMovieSceneMediaSection* NewSection = AddNewSectionOnRow(Time, RowIndex);
	NewSection->SetMediaSource(&MediaSource);
	StartGetDuration(&MediaSource, NewSection);
	return NewSection;
}


UMovieSceneSection* UMovieSceneMediaTrack::AddNewMediaSourceProxyOnRow(UMediaSource* MediaSource, const FMovieSceneObjectBindingID& ObjectBinding, int32 MediaSourceProxyIndex, FFrameNumber Time, int32 RowIndex)
{
	UMovieSceneMediaSection* NewSection = AddNewSectionOnRow(Time, RowIndex);
	NewSection->SetMediaSourceProxy(ObjectBinding, MediaSourceProxyIndex);
	StartGetDuration(MediaSource, NewSection);
	return NewSection;
}


UMovieSceneMediaSection* UMovieSceneMediaTrack::AddNewSectionOnRow(FFrameNumber Time, int32 RowIndex)
{
	const float DefaultMediaSectionDuration = 1.0f;
	FFrameRate TickResolution = GetTypedOuter<UMovieScene>()->GetTickResolution();
	FFrameTime DurationToUse  = DefaultMediaSectionDuration * TickResolution;

	// add the section
	UMovieSceneMediaSection* NewSection = NewObject<UMovieSceneMediaSection>(this, NAME_None, RF_Transactional);

	NewSection->InitialPlacementOnRow(MediaSections, Time, DurationToUse.FrameNumber.Value, RowIndex);

	UMovieSceneTrack::AddSection(FSectionParameter(*NewSection));

	return NewSection;
}

void UMovieSceneMediaTrack::TickOutput()
{
#if WITH_EDITOR
	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "MediaSectionDuration_Transaction", "Set Media Section Duration"));

	// Do we have any new sections that need durations?
	if (NewSections.Num() > 0)
	{
		// Can we get the duration?
		if (bGetDurationDelay)
		{
			bGetDurationDelay = false;
		}
		else
		{
			// Loop over all new sections.
			for (int32 Index = 0; Index < NewSections.Num();)
			{
				if (NewSections[Index].Key.IsValid())
				{
					// Try and get the duration.
					if (GetDuration(NewSections[Index].Key, NewSections[Index].Value))
					{
						NewSections.RemoveAtSwap(Index);
					}
					else
					{
						++Index;
					}
				}
				else
				{
					NewSections.RemoveAtSwap(Index);
				}
			}
		}
	}

	if (NewSections.Num() == 0)
	{
		if (ClockSink.IsValid())
		{
			IMediaModule* MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");
			if (MediaModule != nullptr)
			{
				MediaModule->GetClock().RemoveSink(ClockSink.ToSharedRef());
			}
		}
	}
#endif // WITH_EDITOR
}


/* UMovieSceneTrack interface
 *****************************************************************************/

void UMovieSceneMediaTrack::AddSection(UMovieSceneSection& Section)
{
	MediaSections.Add(&Section);
	UpdateSectionTextureIndices();
}


bool UMovieSceneMediaTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneMediaSection::StaticClass();
}


UMovieSceneSection* UMovieSceneMediaTrack::CreateNewSection()
{
	return NewObject<UMovieSceneMediaSection>(this, NAME_None, RF_Transactional);
}


const TArray<UMovieSceneSection*>& UMovieSceneMediaTrack::GetAllSections() const
{
	return MediaSections;
}


bool UMovieSceneMediaTrack::HasSection(const UMovieSceneSection& Section) const
{
	return MediaSections.Contains(&Section);
}


bool UMovieSceneMediaTrack::IsEmpty() const
{
	return MediaSections.Num() == 0;
}


void UMovieSceneMediaTrack::RemoveSection(UMovieSceneSection& Section)
{
	MediaSections.Remove(&Section);
	UpdateSectionTextureIndices();
}

void UMovieSceneMediaTrack::RemoveSectionAt(int32 SectionIndex)
{
	MediaSections.RemoveAt(SectionIndex);
	UpdateSectionTextureIndices();
}

#if WITH_EDITOR

void UMovieSceneMediaTrack::BeginDestroy()
{
	if (ClockSink.IsValid())
	{
		// Tell sink we are done.
		ClockSink->OwnerDestroyed();

		IMediaModule* MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");
		if (MediaModule != nullptr)
		{
			MediaModule->GetClock().RemoveSink(ClockSink.ToSharedRef());
		}

		ClockSink.Reset();
	}

	Super::BeginDestroy();
}

EMovieSceneSectionMovedResult UMovieSceneMediaTrack::OnSectionMoved(UMovieSceneSection& InSection, const FMovieSceneSectionMovedParams& Params)
{
	UpdateSectionTextureIndices();
	return EMovieSceneSectionMovedResult::None;
}

#endif // WITH_EDITOR

FMovieSceneEvalTemplatePtr UMovieSceneMediaTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneMediaSectionTemplate(*CastChecked<const UMovieSceneMediaSection>(&InSection), *this);
}

void UMovieSceneMediaTrack::UpdateSectionTextureIndices()
{
#if WITH_EDITOR
	// Do we have an object binding?
	const UMovieScene* MovieScene = Cast<UMovieScene>(GetOuter());
	if (MovieScene != nullptr)
	{
		const TArray<FMovieSceneBinding>& Bindings = MovieScene->GetBindings();
		for (const FMovieSceneBinding& Binding : Bindings)
		{
			bool bIsThisMyBinding = false;
			const TArray<UMovieSceneTrack*>& Tracks = Binding.GetTracks();
			for (UMovieSceneTrack* Track : Tracks)
			{
				if ((Track != nullptr) && (Track == this))
				{
					bIsThisMyBinding = true;
					break;
				}
			}

			if (bIsThisMyBinding)
			{
				// Get all sections.
				TArray<UMovieSceneMediaSection*> AllSections;
				for (UMovieSceneTrack* Track : Tracks)
				{
					if (Track != nullptr)
					{
						const TArray<UMovieSceneSection*>& Sections = Track->GetAllSections();
						for (UMovieSceneSection* Section : Sections)
						{
							UMovieSceneMediaSection* MediaSection = Cast<UMovieSceneMediaSection>(Section);
							if (MediaSection != nullptr)
							{
								AllSections.Add(MediaSection);
							}
						}
					}
				}

				// Set up indices from earliest section to latest.
				AllSections.Sort([](UMovieSceneMediaSection& A, UMovieSceneMediaSection& B)
				{ return A.GetRange().GetLowerBoundValue() < B.GetRange().GetLowerBoundValue(); });

				int32 CurrentTextureIndex = 0;
				TSet<int32> AllocatedTextures;
				TArray<UMovieSceneMediaSection*>ActiveSections;

				for (UMovieSceneMediaSection* Section : AllSections)
				{
					TRange <FFrameNumber> Range = Section->GetRange();
					FFrameNumber RangeStart = Range.GetLowerBoundValue();

					// Remove expired sections.
					for (int32 Index = 0; Index < ActiveSections.Num();)
					{
						UMovieSceneMediaSection* ActiveSection = ActiveSections[Index];
						if (ActiveSection->GetRange().GetUpperBoundValue() <= RangeStart)
						{
							if (CurrentTextureIndex > ActiveSection->TextureIndex)
							{
								CurrentTextureIndex = ActiveSection->TextureIndex;
							}
							AllocatedTextures.Remove(ActiveSection->TextureIndex);
							ActiveSections.RemoveAtSwap(Index);
						}
						else
						{
							Index++;
						}
					}
					
					// Set up our section.
					Section->TextureIndex = CurrentTextureIndex;
					AllocatedTextures.Add(CurrentTextureIndex);
					ActiveSections.Add(Section);

					// Get the next texture index to use.
					while (true)
					{
						CurrentTextureIndex++;
						if (AllocatedTextures.Contains(CurrentTextureIndex) == false)
						{
							break;
						}
					}
				}
				break;
			}
		}
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR

void UMovieSceneMediaTrack::StartGetDuration(UMediaSource* MediaSource, UMovieSceneSection* Section)
{
	// Create media player.
	TStrongObjectPtr<UMediaPlayer> MediaPlayer = TStrongObjectPtr<UMediaPlayer>(
		NewObject<UMediaPlayer>(GetTransientPackage(),
			MakeUniqueObjectName(GetTransientPackage(),
				UMediaPlayer::StaticClass())));

	// Open the media.
	MediaPlayer->PlayOnOpen = false;
	FMediaPlayerOptions PlayerOptions;
	PlayerOptions.SetAllAsOptional();
	PlayerOptions.InternalCustomOptions.Emplace(MediaPlayerOptionValues::ParseTimecodeInfo(), FVariant());
	
	if (MediaPlayer->OpenSourceWithOptions(MediaSource, PlayerOptions))
	{
		NewSections.Emplace(MediaPlayer, Section);
	}

	// Some players like Electra report that they are closed at this point, so wait a frame.
	bGetDurationDelay = true;

	// Start the clock sink so we can tick.
	IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");
	if (MediaModule != nullptr)
	{
		if (ClockSink.IsValid() == false)
		{
			ClockSink = MakeShared<FMovieSceneMediaTrackClockSink, ESPMode::ThreadSafe>(this);
		}
		MediaModule->GetClock().AddSink(ClockSink.ToSharedRef());
	}
}

namespace UE::MovieSceneMediaTrack::Private
{
	FString GetTimecodeStringFromMediaPlayer(const UMediaPlayer* InMediaPlayer)
	{
		if (UMediaPlaylist* Playlist = InMediaPlayer->GetPlaylist())
		{
			if (const UMediaSource* MediaSource = Playlist->Get(InMediaPlayer->GetPlaylistIndex()))
			{
				return MediaSource->GetMediaOption(UMediaPlayer::MediaInfoNameStartTimecodeValue.Resolve(), FString());
			}
		}
		return FString();
	}
}

bool UMovieSceneMediaTrack::GetDuration(
	const TStrongObjectPtr<UMediaPlayer>& MediaPlayer, TWeakObjectPtr<UMovieSceneSection>& NewSection)
{
	// Check everything is ok.
	bool bIsDone = (MediaPlayer.IsValid() == false) || (MediaPlayer->HasError()) || (MediaPlayer->IsClosed()) || (NewSection.IsValid() == false);

	if (!bIsDone)
	{
		// Get the duration.
		const FTimespan Duration = MediaPlayer->GetDuration();
		if (Duration != 0)
		{
			// Once it is non zero, then set the length of the section.
			const FFrameRate TickResolution = NewSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
			const FFrameNumber StartFrame = NewSection->GetInclusiveStartFrame();
			const FFrameNumber EndFrame = StartFrame + (Duration.GetTotalSeconds() * TickResolution).FrameNumber;
			NewSection->SetEndFrame(TRangeBound<FFrameNumber>::Exclusive(EndFrame));
			bIsDone = true;

			// Timecode from the source. Either player's meta data or media source.
			using namespace UE::MovieSceneMediaTrack::Private;
			const FVariant Value = MediaPlayer->GetMediaInfo(UMediaPlayer::MediaInfoNameStartTimecodeValue.Resolve());
			const FString TimecodeString = Value.IsEmpty() ? GetTimecodeStringFromMediaPlayer(MediaPlayer.Get()) : Value.GetValue<FString>();  
			if (!TimecodeString.IsEmpty())
			{
				const TOptional<FTimecode> Timecode = FTimecode::ParseTimecode(*TimecodeString);
				if (Timecode.IsSet())
				{
					NewSection->TimecodeSource = FMovieSceneTimecodeSource(Timecode.GetValue());
				}
			}
		}
	}

	return bIsDone;
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

