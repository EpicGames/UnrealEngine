// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieScenePropertyTemplate.h"
#include "MediaSampleQueue.h"
#include "Misc/Timespan.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API MEDIACOMPOSITING_API

class FMovieSceneMediaPlayerStore;
class IMediaPlayerProxy;
class UMediaPlayer;
class UMediaTexture;
enum class EMediaEvent;

/** Playback parameters to update the media player. */ 
struct FMovieSceneMediaPlaybackParams
{
	/**
	 * Indicate if player is looping from the corresponding media section parameter.
	 * We need to store this here to have it in HandleMediaPlayerEvent.
	 */
	bool bIsLooping = false;

	/**
	 * If specified, this is the playback time range (in player's time) calculated from the section and sequencer.
	 * We need to store this here to have it in HandleMediaPlayerEvent.
	 */
	TRange<FTimespan> SectionTimeRange;

	/**
	 * Sequencer frame duration used for range tolerance calculations.
	 */
	FTimespan FrameDuration;
};

/**
 * Persistent data that's stored for each currently evaluating section.
 */
struct FMovieSceneMediaData
	: PropertyTemplate::FSectionData
{
	/** Default constructor. */
	UE_API FMovieSceneMediaData();

	/** Virtual destructor. */
	UE_API virtual ~FMovieSceneMediaData() override;

public:

	/**
	 * Get the media player used by this persistent data.
	 *
	 * @return The currently used media player, if any.
	 */
	UE_API UMediaPlayer* GetMediaPlayer();

	/**
	 * Get the optional proxy object used by this persistent data.
	 */
	UObject* GetPlayerProxy() { return PlayerProxy.Get(); }

	/**
	 * Returns true if the player proxy object has become stale.
	 */
	bool IsPlayerProxyStale() { return PlayerProxy.IsStale(); }

	/**
	 * Get the layer index we are using (when using a proxy).
	 */
	int32 GetProxyLayerIndex() { return ProxyLayerIndex; }

	/**
	 * Get the texture index we are using (when using a proxy).
	 */
	int32 GetProxyTextureIndex() { return ProxyTextureIndex; }

	/**
	 * Set the time to seek to after opening a media source has finished.
	 *
	 * @param Time The time to seek to.
	 */
	UE_API void SeekOnOpen(FTimespan Time);

	UE_DEPRECATED(5.6, "Use new Setup function with the playback parameters")
	UE_API void Setup(const TSharedPtr<FMovieSceneMediaPlayerStore>& InMediaPlayerStore, const UObject* InSection, UMediaPlayer* OverrideMediaPlayer, UObject* InPlayerProxy, int32 InProxyLayerIndex, int32 InProxyTextureIndex);
	
	/** Set up this persistent data object. */
	UE_API void Setup(const TSharedPtr<FMovieSceneMediaPlayerStore>& InMediaPlayerStore, const UObject* InSection, UMediaPlayer* OverrideMediaPlayer, UObject* InPlayerProxy, int32 InProxyLayerIndex, int32 InProxyTextureIndex, const FMovieSceneMediaPlaybackParams& InPlaybackParams);

	/** Update the player proxy interface object. */
	UE_API void UpdatePlayerProxy(UObject* InPlayerProxy);

	/**
	 * Called from FMovieSceneMediaSectionTemplate::Initialize.
	 */
	UE_API void Initialize(bool bIsEvaluating);

	/**
	 * Called from FMovieSceneMediaSectionTemplate::TearDown.
	 */
	UE_API void TearDown();

	/** Retrieve the sample queue and release the ownership. */
	UE_API TSharedPtr<FMediaTextureSampleQueue> TransferSampleQueue();

	/** Get the proxy media texture. */
	UE_API UMediaTexture* GetProxyMediaTexture();
	
	/**
	 * Stores if the aspect ratio has been set yet.
	 */
	bool bIsAspectRatioSet;

	/**
	 * Indicate if the section template has been executed already or not.
	 * This is used to detect when a evaluation template was rebuilt and has potentially changed.
	 */
	bool bHasBeenExecuted = false;
	
private:
	/**
	 * Does the work needed so we can use our proxy media texture.
	 */
	UE_API void StartUsingProxyMediaTexture();

	/**
	 * Does the work needed when we no longer use our proxy media texture.
	 */
	UE_API void StopUsingProxyMediaTexture();

	/** Callback for media player events. */
	UE_API void HandleMediaPlayerEvent(EMediaEvent Event);

private:
	bool bOverrideMediaPlayer;

	/** The media player used by this object. */
	UMediaPlayer* MediaPlayer;
	/** Optional proxy for the media player. */
	TWeakObjectPtr<UObject> PlayerProxy;
	/** Media texture allocated from the proxy. */
	TWeakObjectPtr<UMediaTexture> ProxyMediaTexture;
	/** Layer that this section should reside in. */
	int32 ProxyLayerIndex;
	/** Index of texture allocated from the proxy. */
	int32 ProxyTextureIndex;

	/**
	 * Sample queue to be used as video sink for the media player.
	 * It is owned by the media section because it must be set in the player prior to
	 * entering the playback section, i.e. in preroll. We can't ue the internal media texture
	 * sample queue because it is in use by another section during this section's preroll.
	 * Each section needs it's sample queue independently of the media texture.
	 */
	TSharedPtr<FMediaTextureSampleQueue> SampleQueue;
	
	/** The time to seek to after the media source is opened. */
	FTimespan SeekOnOpenTime;

	/** Pointer to the media player store this section data was setup with. Will be used to return the media player to the store on destruction. */
	TWeakPtr<FMovieSceneMediaPlayerStore> MediaPlayerStoreWeak;

	/** Owning Media Section. Used as persistent identifier for media player data. */
	FObjectKey MediaSection;

	/** Additional player parameters to set before the first seek. */
	FMovieSceneMediaPlaybackParams PlaybackParams;
};

#undef UE_API
