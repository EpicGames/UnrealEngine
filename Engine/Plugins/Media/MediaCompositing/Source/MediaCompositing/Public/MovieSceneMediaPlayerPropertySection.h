// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSection.h"
#include "MovieSceneMediaPlayerPropertySection.generated.h"

class UMediaSource;

/**
 * Implements a movie scene section for media playback on a UMediaPlayer.
 */
UCLASS(MinimalAPI)
class UMovieSceneMediaPlayerPropertySection
	: public UMovieSceneSection
{
public:

	GENERATED_BODY()

	/** The source to play with this video track. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Media")
	TObjectPtr<UMediaSource> MediaSource;

	/** Whether to loop this video. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Media")
	bool bLoop;

	UMovieSceneMediaPlayerPropertySection(const FObjectInitializer& ObjectInitializer);
};
