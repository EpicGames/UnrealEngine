// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"

#include "MovieSceneMediaPlayerPropertyTrack.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneMediaPlayerPropertyTrack
	: public UMovieScenePropertyTrack
	, public IMovieSceneTrackTemplateProducer
{
public:

	GENERATED_BODY()

	UMovieSceneMediaPlayerPropertyTrack(const FObjectInitializer& ObjectInitializer);

public:

	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;
};
