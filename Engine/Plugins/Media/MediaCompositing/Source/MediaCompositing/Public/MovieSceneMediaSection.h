// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSection.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "MediaSource.h"
#include "MovieSceneObjectBindingID.h"
#include "Channels/MovieSceneBoolChannel.h"

#include "MovieSceneMediaSection.generated.h"

#define UE_API MEDIACOMPOSITING_API

class IMovieScenePlayer;
class UMediaPlayer;
class UMediaSoundComponent;
class UMediaSource;
class UMediaTexture;
struct FMovieSceneSequenceID;

/**
 * Implements a movie scene section for media playback.
 */
UCLASS(MinimalAPI)
class UMovieSceneMediaSection
	: public UMovieSceneSection
{
public:

	GENERATED_BODY()


public:
	/** The source to play with this video track if MediaSourceProxy is not available. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Media")
	TObjectPtr<UMediaSource> MediaSource;
	
	/** The index to pass to MediaSourceProxy to get the media source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media")
	int32 MediaSourceProxyIndex;

	/** Should the media player be set to loop? This can be helpful for media formats that can use this information (such as exr sequences) to pre-cache the starting data when nearing the end of playback. Does not cause the media to continue playing after the end of the section is reached. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media")
	bool bLooping;

	/** Offset into the source media. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Media")
	FFrameNumber StartFrameOffset;

	/** The media texture that receives the track's video output. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Media", meta = (EditCondition = "!bUseExternalMediaPlayer"))
	TObjectPtr<UMediaTexture> MediaTexture;

	/** The media sound component that receives the track's audio output. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Media")
	TObjectPtr<UMediaSoundComponent> MediaSoundComponent;

	/** If true, this track will control a previously created media player instead of automatically creating one. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Media", AdvancedDisplay)
	bool bUseExternalMediaPlayer;

	/** The external media player this track should control. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Media", meta = (EditCondition = "bUseExternalMediaPlayer"), AdvancedDisplay)
	TObjectPtr<UMediaPlayer> ExternalMediaPlayer;

	/** Override the default cache settings. Not used if we have a player proxy as the settings come from the proxy instead. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (EditCondition = "!bHasMediaPlayerProxy", HideEditConditionToggle = true, EditConditionHides), AdvancedDisplay)
	FMediaSourceCacheSettings CacheSettings;

	/** If using an object like a MediaPlate, then this determines which texture to use for crossfading purposes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (EditCondition = "bHasMediaPlayerProxy", HideEditConditionToggle = true, EditConditionHides), AdvancedDisplay)
	int32 TextureIndex;

	/** If true, the specified framerate alignment will be used instead of the media's frame rate. Use this in case the automatic media frame rate is inaccurate. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Media", meta = (InlineEditConditionToggle), AdvancedDisplay)
	bool bManualFrameRateAlignment;

	/** Specify the frame rate for internal presentation frame alignment (i.e. frame duration used for Block On Range calculation). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (EditCondition = "bManualFrameRateAlignment"), AdvancedDisplay)
	FFrameRate FrameRateAlignment;
	
	/** True if the object bound to this track has a media player proxy. */
	UPROPERTY(Transient)
	bool bHasMediaPlayerProxy;

	/** If true then the media player can be open. */
	UPROPERTY()
	FMovieSceneBoolChannel ChannelCanPlayerBeOpen;

public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param ObjectInitializer The object initializer.
	 */
	UE_API UMovieSceneMediaSection(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void PostInitProperties() override;

protected:

	//~ UMovieSceneSection interface
	UE_API virtual void TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys) override;
	UE_API virtual UMovieSceneSection* SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys) override;
	UE_API virtual TOptional<FFrameTime> GetOffsetTime() const override;
	UE_API virtual void MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate) override;
	UE_API virtual void OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) override;
	UE_API virtual void GetReferencedBindings(TArray<FGuid>& OutBindings) override;
	UE_API virtual UObject* GetSourceObject() const override;

public:

	/**
	 * Get this section's video source.
	 *
	 * @return The media source object.
	 * @see SetMediaSource
	 */
	UE_API UMediaSource* GetMediaSource() const;

	/**
	 * Set this section's video source.
	 *
	 * @param InMediaSource The media source object to set.
	 * @see GetMediaSource
	 */
	void SetMediaSource(UMediaSource* InMediaSource)
	{
		MediaSource = InMediaSource;
	}

	/**
	 * Get this section's video source proxy.
	 */
	const FMovieSceneObjectBindingID& GetMediaSourceProxy() const
	{
		return MediaSourceProxyBindingID;
	}

	/**
	 * Set this section's video source proxy.
	 *
	 * @param InObjectBinding		The object this binds to should implement IMediaPlayerProxyInterface.
	 * @param InIndex				Index to pass to IMediaPlayerProxyInterface.
	 */
	void SetMediaSourceProxy(const FMovieSceneObjectBindingID& InObjectBinding, int32 InIndex)
	{
		MediaSourceProxyBindingID = InObjectBinding;
		MediaSourceProxyIndex = InIndex;
	}

	/**
	 * If there is a proxy, get the media source from it.
	 * Otherwise return our media source.
	 * 
	 * @param Player				Player used to get the proxy object from the binding.
	 * @param SequenceID			ID used to get the proxy object from the binding.
	 */
	UE_API UMediaSource* GetMediaSourceOrProxy(IMovieScenePlayer& Player, FMovieSceneSequenceID SequenceID) const;

	/**
	 * Static version of GetMediaSourceOrProxy.
	 *
	 * @param Player				Player used to get the proxy object from the binding.
	 * @param SequenceID			ID used to get the proxy object from the binding.
	 */
	static UE_API UMediaSource* GetMediaSourceOrProxy(IMovieScenePlayer& InPlayer, FMovieSceneSequenceID InSequenceID, UMediaSource* InMediaSource, const FMovieSceneObjectBindingID& InMediaSourceProxyBindingID, int32 InMediaSourceProxyIndex);


#if WITH_EDITORONLY_DATA

	/** @return The thumbnail reference frame offset from the start of this section */
	float GetThumbnailReferenceOffset() const
	{
		return ThumbnailReferenceOffset;
	}

	/** Set the thumbnail reference offset */
	void SetThumbnailReferenceOffset(float InNewOffset)
	{
		Modify();
		ThumbnailReferenceOffset = InNewOffset;
	}

#endif //WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA

	/** The reference frame offset for single thumbnail rendering */
	UPROPERTY()
	float ThumbnailReferenceOffset;

#endif //WITH_EDITORONLY_DATA

private:

	/** The object to get the source to play from if you don't want to directly specify a media source. */
	UPROPERTY(EditAnywhere, Category = "Section")
	FMovieSceneObjectBindingID MediaSourceProxyBindingID;

};

#undef UE_API
