// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "MovieSceneNameableTrack.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StrongObjectPtr.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"

#include "MovieSceneMediaTrack.generated.h"

#define UE_API MEDIACOMPOSITING_API

class FMovieSceneMediaTrackClockSink;
class UMediaPlayer;
class UMediaSource;
class UMovieSceneMediaSection;
struct FMovieSceneObjectBindingID;


/**
 * Implements a movie scene track for media playback.
 */
UCLASS(MinimalAPI)
class UMovieSceneMediaTrack
	: public UMovieSceneNameableTrack
	, public IMovieSceneTrackTemplateProducer
{
public:

	GENERATED_BODY()

	/**
	 * Create and initialize a new instance.
	 *
	 * @param ObjectInitializer The object initializer.
	 */
	UE_API UMovieSceneMediaTrack(const FObjectInitializer& ObjectInitializer);

public:

	/** Adds a new media source to the track. */
	UE_API virtual UMovieSceneSection* AddNewMediaSourceOnRow(UMediaSource& MediaSource, FFrameNumber Time, int32 RowIndex);
	/** Adds a new media source to the track. */
	UE_API virtual UMovieSceneSection* AddNewMediaSourceProxyOnRow(UMediaSource* MediaSource, const FMovieSceneObjectBindingID& ObjectBinding, int32 MediaSourceProxyIndex, FFrameNumber Time, int32 RowIndex);

	/** Adds a new media source on the next available/non-overlapping row. */
	virtual UMovieSceneSection* AddNewMediaSource(UMediaSource& MediaSource, FFrameNumber Time) { return AddNewMediaSourceOnRow(MediaSource, Time, INDEX_NONE); }
	/** Adds a new media source on the next available/non-overlapping row. */
	virtual UMovieSceneSection* AddNewMediaSourceProxy(UMediaSource* MediaSource, const FMovieSceneObjectBindingID& ObjectBinding, int32 MediaSourceProxyIndex, FFrameNumber Time)
	{
		return AddNewMediaSourceProxyOnRow(MediaSource, ObjectBinding, MediaSourceProxyIndex, Time, INDEX_NONE);
	}

	/**
	 * Called from the media clock.
	 */
	UE_API void TickOutput();

public:

	//~ UMovieSceneTrack interface

	UE_API virtual void AddSection(UMovieSceneSection& Section) override;
	UE_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	UE_API virtual UMovieSceneSection* CreateNewSection() override;
	UE_API virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;
	UE_API virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	UE_API virtual bool HasSection(const UMovieSceneSection& Section) const override;
	UE_API virtual bool IsEmpty() const override;
	UE_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	UE_API virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool SupportsMultipleRows() const override { return true; }


#if WITH_EDITOR

	UE_API virtual void BeginDestroy() override;
	
	UE_API virtual EMovieSceneSectionMovedResult OnSectionMoved(UMovieSceneSection& Section, const FMovieSceneSectionMovedParams& Params);

#endif // WITH_EDITOR

	/**
	 * Force synchronous frame requests *while* manually scrubbing the media track, at the cost of blocking the editor.
	 * This can be valuable for always maintaining alignment of media & CG while working.
	 */
	UPROPERTY(EditAnywhere, Category = "Media Track")
	bool bSynchronousScrubbing = false;

private:
	/** Base function to add a new section. */
	UE_API UMovieSceneMediaSection* AddNewSectionOnRow(FFrameNumber Time, int32 RowIndex);

	/** List of all media sections. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> MediaSections;

#if WITH_EDITORONLY_DATA

	/** List of players that are we are trying to get durations from for the corresponding sections. */
	TArray<TPair<TStrongObjectPtr<UMediaPlayer>, TWeakObjectPtr<UMovieSceneSection>>> NewSections;
	/** Our media clock sink. */
	TSharedPtr<FMovieSceneMediaTrackClockSink, ESPMode::ThreadSafe> ClockSink;
	/** If true then don't check for the duration this frame. */
	bool bGetDurationDelay;

#endif // WITH_EDITORONLY_DATA

	/**
	 * Updates all our sections so they have correct texture indices
	 * so the proxy can blend all the sections together correctly.
	 */
	UE_API void UpdateSectionTextureIndices();

#if WITH_EDITOR
	/**
	 * Starts the process to get the duration of the media.
	 * It might take a frame or more.
	 *
	 * @param MediaSource		Media to inspect.
	 * @param Section			Will set this sequencer section to the length of the media.
	*/
	UE_API void StartGetDuration(UMediaSource* MediaSource, UMovieSceneSection* Section);

	/**
	 * Call this after StartGetDuration to try and get the duration of the media.
	 *
	 * @param MediaPlayer		Player that is opening the media.
	 * @param NewSection		Movie section will be set to the media duration.
	 * @return True if it is done and the player can be removed.
	 */
	UE_API bool GetDuration(const TStrongObjectPtr<UMediaPlayer>& MediaPlayer,
		TWeakObjectPtr<UMovieSceneSection>& NewSection);
#else  // WITH_EDITOR
	void StartGetDuration(UMediaSource* MediaSource, UMovieSceneSection* Section) {}
#endif // WITH_EDITOR

};

#undef UE_API
