// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MediaCompositingEditor : ModuleRules
	{
		public MediaCompositingEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"ContentBrowser",
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"EditorFramework",
					"EditorWidgets",
					"Engine",
					"ImageCore",
					"ImgMedia",
					"InputCore",
					"LevelSequence",
					"MediaAssets",
					"MediaCompositing",
					"MediaUtils",
					"MovieScene",
					"MovieSceneTracks",
					"RenderCore",
					"RHI",
					"SequencerCore",
					"Sequencer",
					"SequenceRecorder",
					"Slate",
					"SlateCore",
					"UnrealEd",
					"TimeManagement"
				});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"MovieSceneTools",
				});
		}
	}
}
