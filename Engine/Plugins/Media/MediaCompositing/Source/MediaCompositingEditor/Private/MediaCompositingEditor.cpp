// Copyright Epic Games, Inc. All Rights Reserved.

#include "Delegates/IDelegateInstance.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"

#include "ISequenceRecorder.h"
#include "Shared/MediaCompositingEditorStyle.h"
#include "Sequencer/MediaSequenceRecorderExtender.h"
#include "Sequencer/MediaTrackEditor.h"
#include "Sequencer/MediaPlayerPropertyTrackEditor.h"

#define LOCTEXT_NAMESPACE "MediaCompositingEditorModule"

/**
 * Implements the MediaCompositing module.
 */
class FMediaCompositingEditorModule
	: public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
		FMediaCompositingEditorStyle::Get();

		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		TrackEditorBindingHandle = SequencerModule.RegisterPropertyTrackEditor<FMediaTrackEditor>();
		PropertyTrackEditorBindingHandle = SequencerModule.RegisterPropertyTrackEditor<FMediaPlayerPropertyTrackEditor>();

		ISequenceRecorder& SequenceRecorder = FModuleManager::LoadModuleChecked<ISequenceRecorder>("SequenceRecorder");
		RecorderExtender = MakeShared<FMediaSequenceRecorderExtender>();
		SequenceRecorder.AddSequenceRecorderExtender(RecorderExtender);
	}
	
	virtual void ShutdownModule() override
	{
		ISequenceRecorder* SequenceRecorder = FModuleManager::Get().GetModulePtr<ISequenceRecorder>("SequenceRecorder");
		if (SequenceRecorder && RecorderExtender.IsValid())
		{
			SequenceRecorder->RemoveSequenceRecorderExtender(RecorderExtender);
		}
		RecorderExtender.Reset();

		FMediaCompositingEditorStyle::Destroy();

		ISequencerModule* SequencerModulePtr = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer");	
		
		if (SequencerModulePtr)
		{
			SequencerModulePtr->UnRegisterTrackEditor(TrackEditorBindingHandle);
			SequencerModulePtr->UnRegisterTrackEditor(PropertyTrackEditorBindingHandle);
		}
	}

	FDelegateHandle TrackEditorBindingHandle;
	FDelegateHandle PropertyTrackEditorBindingHandle;
	TSharedPtr<FMediaSequenceRecorderExtender> RecorderExtender;
};


IMPLEMENT_MODULE(FMediaCompositingEditorModule, MediaCompositingEditor);


#undef LOCTEXT_NAMESPACE
