// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlayerPropertyTrackEditor.h"

#include "ContentBrowserModule.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "ISequencer.h"
#include "SequencerSectionPainter.h"
#include "MovieSceneMediaPlayerPropertySection.h"
#include "MovieSceneMediaPlayerPropertyTrack.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "MediaSource.h"
#include "LevelSequence.h"
#include "SequencerSettings.h"

#include "Sequencer/MediaThumbnailSection.h"

#define LOCTEXT_NAMESPACE "FMediaPlayerPropertyTrackEditor"


struct FMediaPlayerPropertySection : FSequencerSection
{
	FMediaPlayerPropertySection(UMovieSceneMediaPlayerPropertySection* InSection)
		: FSequencerSection(*InSection)
	{}

	virtual FMargin GetContentPadding() const override
	{
		return FMargin(8.0f, 15.0f);
	}

	virtual float GetSectionHeight() const override
	{
		const float InnerHeight = FAppStyle::GetFontStyle("NormalFont").Size + 8.f;
		return InnerHeight + 2 * 9.0f; // make space for the film border
	}

	virtual FText GetSectionTitle() const override
	{
		UMovieSceneMediaPlayerPropertySection* MediaSection = CastChecked<UMovieSceneMediaPlayerPropertySection>(WeakSection.Get());
		if (MediaSection->MediaSource == nullptr)
		{
			return LOCTEXT("NoSequence", "Empty");
		}

		return FText::FromString(MediaSection->MediaSource->GetFName().ToString());
	}

	virtual int32 OnPaintSection(FSequencerSectionPainter& InPainter) const override
	{
		// draw background
		InPainter.LayerId = InPainter.PaintSectionBackground();

		FVector2D SectionSize = InPainter.SectionGeometry.GetLocalSize();
		FSlateClippingZone ClippingZone(InPainter.SectionClippingRect.InsetBy(FMargin(1.0f)));
		
		InPainter.DrawElements.PushClip(ClippingZone);
		{
			static const FSlateBrush* FilmBorder = FAppStyle::GetBrush("Sequencer.Section.FilmBorder");

			// draw top film border
			FSlateDrawElement::MakeBox(
				InPainter.DrawElements,
				InPainter.LayerId++,
				InPainter.SectionGeometry.ToPaintGeometry(FVector2D(SectionSize.X - 2.0f, 7.0f), FSlateLayoutTransform(FVector2D(1.0f, 4.0f))),
				FilmBorder,
				InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect
			);

			// draw bottom film border
			FSlateDrawElement::MakeBox(
				InPainter.DrawElements,
				InPainter.LayerId++,
				InPainter.SectionGeometry.ToPaintGeometry(FVector2D(SectionSize.X - 2.0f, 7.0f), FSlateLayoutTransform(FVector2D(1.0f, SectionSize.Y - 11.0f))),
				FilmBorder,
				InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect
			);
		}
		InPainter.DrawElements.PopClip();

		return InPainter.LayerId;
	}
};



void FMediaPlayerPropertyTrackEditor::OnAnimatedPropertyChanged(const FPropertyChangedParams& PropertyChangedParams)
{
	// Get the movie scene we want to autokey
	UMovieSceneSequence* MovieSceneSequence = GetMovieSceneSequence();
	if (!MovieSceneSequence)
	{
		return;
	}

	if (PropertyChangedParams.KeyMode != ESequencerKeyMode::ManualKey && PropertyChangedParams.KeyMode != ESequencerKeyMode::ManualKeyForced)
	{
		return;
	}

	const FProperty* Property = PropertyChangedParams.PropertyPath.GetLeafMostProperty().Property.Get();
	if (!Property)
	{
		return;
	}

	MovieSceneSequence->SetFlags(RF_Transactional);
	FScopedTransaction Transaction( LOCTEXT("AddMediaPropertyTrack", "Add Media Property Track"));

	FName UniqueName(*PropertyChangedParams.PropertyPath.ToString(TEXT(".")));

	for (UObject* Object : PropertyChangedParams.ObjectsThatChanged)
	{
		FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(Object);
		if (HandleResult.Handle.IsValid())
		{
			// Ensure it has a track
			FFindOrCreateTrackResult TrackResult = FindOrCreateTrackForObject(HandleResult.Handle, UMovieSceneMediaPlayerPropertyTrack::StaticClass(), Property->GetFName());
			if (TrackResult.bWasCreated)
			{
				InitializeNewTrack(CastChecked<UMovieSceneMediaPlayerPropertyTrack>(TrackResult.Track), PropertyChangedParams);

				GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
			}
		}
	}
}

FText FMediaPlayerPropertyTrackEditor::GetDisplayName() const
{
	return LOCTEXT("MediaPlayerPropertyTrackEditor_DisplayName", "Media Player Property");
}

UMovieSceneTrack* FMediaPlayerPropertyTrackEditor::AddTrack(UMovieScene* FocusedMovieScene, const FGuid& ObjectHandle, TSubclassOf<class UMovieSceneTrack> TrackClass, FName UniqueTypeName)
{
	UMovieSceneTrack* NewTrack = FPropertyTrackEditor::AddTrack(FocusedMovieScene, ObjectHandle, TrackClass, UniqueTypeName);
	UMovieSceneMediaPlayerPropertyTrack* MediaPlayerTrack = Cast<UMovieSceneMediaPlayerPropertyTrack>(NewTrack);
	return NewTrack;
}

TSharedPtr<SWidget> FMediaPlayerPropertyTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	UMovieSceneMediaPlayerPropertyTrack* MediaTrack = Cast<UMovieSceneMediaPlayerPropertyTrack>(Track);
	if (!MediaTrack)
	{
		return SNullWidget::NullWidget;
	}

	auto CreatePicker = [this, MediaTrack]
	{
		TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
		UMovieSceneSequence* Sequence = SequencerPtr.IsValid() ? SequencerPtr->GetFocusedMovieSceneSequence() : nullptr;

		FAssetPickerConfig AssetPickerConfig;
		{
			AssetPickerConfig.OnAssetSelected     = FOnAssetSelected::CreateRaw(this,     &FMediaPlayerPropertyTrackEditor::AddNewSection,             MediaTrack);
			AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateRaw(this, &FMediaPlayerPropertyTrackEditor::AddNewSectionEnterPressed, MediaTrack);
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.bAddFilterUI = true;
			AssetPickerConfig.bShowTypeInColumnView = false;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
			AssetPickerConfig.Filter.bRecursiveClasses = true;
			AssetPickerConfig.Filter.ClassPaths.Add(UMediaSource::StaticClass()->GetClassPathName());
			AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
			AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));
		}

		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
		const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

		TSharedRef<SBox> Picker = SNew(SBox)
			.WidthOverride(WidthOverride)
			.HeightOverride(HeightOverride)
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			];

		return Picker;
	};

	return UE::Sequencer::MakeAddButton(LOCTEXT("AddMediaSection_Text", "Media"), FOnGetContent::CreateLambda(CreatePicker), Params.ViewModel);
}

TSharedRef<ISequencerSection> FMediaPlayerPropertyTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	return MakeShared<FMediaPlayerPropertySection>(CastChecked<UMovieSceneMediaPlayerPropertySection>(&SectionObject));
}

bool FMediaPlayerPropertyTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneMediaPlayerPropertyTrack::StaticClass()) : ETrackSupport::Default;

	if (TrackSupported == ETrackSupport::NotSupported)
	{
		return false;
	}

	return (InSequence && InSequence->IsA(ULevelSequence::StaticClass())) || TrackSupported == ETrackSupport::Supported;
}

const FSlateBrush* FMediaPlayerPropertyTrackEditor::GetIconBrush() const
{
	return FAppStyle::GetBrush("Sequencer.Tracks.Media");
}

void FMediaPlayerPropertyTrackEditor::AddNewSection(const FAssetData& AssetData, UMovieSceneMediaPlayerPropertyTrack* MediaTrack)
{
	FSlateApplication::Get().DismissAllMenus();

	UObject* SelectedObject = AssetData.GetAsset();
	if (SelectedObject)
	{
		UMediaSource* MediaSource = CastChecked<UMediaSource>(AssetData.GetAsset());

		const FScopedTransaction Transaction(LOCTEXT("AddMedia_Transaction", "Add Media"));

		MediaTrack->Modify();

		TArray<UMovieSceneSection*> Sections = MediaTrack->GetAllSections();

		const float DefaultMediaSectionDuration = 5.0f;
		FFrameRate TickResolution = MediaTrack->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameTime DurationToUse  = DefaultMediaSectionDuration * TickResolution;

		// add the section
		UMovieSceneMediaPlayerPropertySection* NewSection = CastChecked<UMovieSceneMediaPlayerPropertySection>(MediaTrack->CreateNewSection());

		NewSection->InitialPlacement(MediaTrack->GetAllSections(), GetSequencer()->GetLocalTime().Time.FrameNumber, DurationToUse.FrameNumber.Value, false);
		NewSection->MediaSource = MediaSource;

		MediaTrack->AddSection(*NewSection);

		GetSequencer()->EmptySelection();
		GetSequencer()->SelectSection(NewSection);
		GetSequencer()->ThrobSectionSelection();

		GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	}
}

void FMediaPlayerPropertyTrackEditor::AddNewSectionEnterPressed(const TArray<FAssetData>& AssetData, UMovieSceneMediaPlayerPropertyTrack* Track)
{
	if (AssetData.Num() > 0)
	{
		AddNewSection(AssetData[0].GetAsset(), Track);
	}
}

#undef LOCTEXT_NAMESPACE
