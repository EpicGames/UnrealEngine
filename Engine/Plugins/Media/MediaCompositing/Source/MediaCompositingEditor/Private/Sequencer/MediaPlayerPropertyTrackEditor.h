// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyTrackEditor.h"
#include "MovieSceneMediaPlayerPropertyTrack.h"
#include "MediaPlayer.h"

struct FAssetData;
class ISequencer;
class UMediaSource;
class UMovieSceneMediaPlayerPropertyTrack;

/**
 * Track editor that understands how to animate MediaPlayer properties on objects
 */
class FMediaPlayerPropertyTrackEditor
	: public FPropertyTrackEditor<UMovieSceneMediaPlayerPropertyTrack>
{
public:

	FMediaPlayerPropertyTrackEditor(TSharedRef<ISequencer> InSequencer)
		: FPropertyTrackEditor(InSequencer, GetAnimatedPropertyTypes())
	{}

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
	{
		return MakeShared<FMediaPlayerPropertyTrackEditor>(OwningSequencer);
	}


	static TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes()
	{
		return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>({ FAnimatedPropertyKey::FromObjectType(UMediaPlayer::StaticClass()) });
	}

private:

	virtual FText GetDisplayName() const override;
	virtual UMovieSceneTrack* AddTrack(UMovieScene* FocusedMovieScene, const FGuid& ObjectHandle, TSubclassOf<class UMovieSceneTrack> TrackClass, FName UniqueTypeName) override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual void OnAnimatedPropertyChanged(const FPropertyChangedParams& PropertyChangedParams) override;

	virtual void GenerateKeysFromPropertyChanged(const FPropertyChangedParams&, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys&) override {}

	void AddNewSection(const FAssetData& AssetData, UMovieSceneMediaPlayerPropertyTrack* MediaTrack);
	void AddNewSectionEnterPressed(const TArray<FAssetData>& AssetData, UMovieSceneMediaPlayerPropertyTrack* Track);
};
