// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaPlayerRecording.h"


#include "LevelSequence.h"
#include "MediaPlayer.h"
#include "MediaSequenceRecorderExtender.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "MovieSceneMediaPlayerSectionRecorder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaPlayerRecording)

static const FName SequencerActorTag(TEXT("SequencerActor"));
static const FName MovieSceneSectionRecorderFactoryName("MovieSceneSectionRecorderFactory");

UMediaPlayerRecording::UMediaPlayerRecording(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RecordingSettings.bActive = true;
}

bool UMediaPlayerRecording::StartRecording(ULevelSequence* CurrentSequence, float CurrentSequenceTime, const FString& BaseAssetPath, const FString& SessionName)
{
	const UMediaSequenceRecorderSettings* Settings = GetDefault<UMediaSequenceRecorderSettings>();
	if (!Settings->bRecordMediaPlayerEnabled)
	{
		return false;
	}
	if (!IsActive())
	{
		return false;
	}

	UMediaPlayer* MediaPlayer = GetMediaPlayerToRecord();
	if (!MediaPlayer)
	{
		return false;
	}


	FDirectoryPath MediaDirectory;
	MediaDirectory.Path = BaseAssetPath;
	if (Settings->MediaPlayerSubDirectory.Len())
	{
		MediaDirectory.Path = FPaths::Combine(MediaDirectory.Path, Settings->MediaPlayerSubDirectory);
	}
	MediaDirectory.Path = FPaths::Combine(MediaDirectory.Path, GetMediaPlayerToRecord()->GetName());

	TSharedPtr<FMovieSceneMediaPlayerSectionRecorder> Recorder = MakeShareable(new FMovieSceneMediaPlayerSectionRecorder(RecordingSettings, MediaDirectory.Path));
	Recorder->CreateSection(GetMediaPlayerToRecord(), CurrentSequence->GetMovieScene(), FGuid::NewGuid(), 0.0f);
	Recorder->Record(0.0f);
	SectionRecorders.Add(Recorder);

	return true;
}

void UMediaPlayerRecording::Tick(ULevelSequence* CurrentSequence, float CurrentSequenceTime)
{
	if (IsRecording())
	{
		for(auto& SectionRecorder : SectionRecorders)
		{
			SectionRecorder->Record(CurrentSequenceTime);
		}
	}
}

bool UMediaPlayerRecording::StopRecording(ULevelSequence* OriginalSequence, float CurrentSequenceTime)
{
	if (!GetDefault<UMediaSequenceRecorderSettings>()->bRecordMediaPlayerEnabled)
	{
		return false;
	}
	if (!RecordingSettings.bActive)
	{
		return false;
	}

	UMediaPlayer* MediaPlayer = GetMediaPlayerToRecord();
	if (!MediaPlayer)
	{
		return false;
	}

	FScopedSlowTask SlowTask((float)SectionRecorders.Num() + 1.0f, FText::Format(NSLOCTEXT("SequenceRecorder", "ProcessingMedia", "Processing Media {0}"), FText::FromName(MediaPlayer->GetFName())));

	// stop property recorders
	for(auto& SectionRecorder : SectionRecorders)
	{
		SlowTask.EnterProgressFrame();

		SectionRecorder->FinalizeSection(CurrentSequenceTime);
	}

	SlowTask.EnterProgressFrame();

	SectionRecorders.Empty();

	return true;
}

bool UMediaPlayerRecording::IsRecording() const
{
	return MediaPlayerToRecord.IsValid() && SectionRecorders.Num() > 0;
}

UMediaPlayer* UMediaPlayerRecording::GetMediaPlayerToRecord() const
{
	return MediaPlayerToRecord.Get();
}

void UMediaPlayerRecording::SetMediaPlayerToRecord(UMediaPlayer* InMediaPlayer)
{
	MediaPlayerToRecord = InMediaPlayer;
}

