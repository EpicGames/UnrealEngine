// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequenceRecordingBase.h"

#include "Engine/EngineTypes.h"
#include "IMovieSceneSectionRecorder.h"
#include "MediaPlayer.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MediaPlayerRecording.generated.h"

class ULevelSequence;

UENUM()
enum class EMediaPlayerRecordingNumerationStyle : uint8
{
	AppendFrameNumber,
	AppendSampleTime
};

UENUM()
enum class EMediaPlayerRecordingImageFormat : uint8
{
	PNG,
	JPEG,
	BMP,
	EXR,
};

USTRUCT()
struct FMediaPlayerRecordingSettings
{
	GENERATED_USTRUCT_BODY()

	FMediaPlayerRecordingSettings()
		: bActive(true)
		, bRecordMediaFrame(false)
		, BaseFilename(TEXT("Frame"))
		, NumerationStyle(EMediaPlayerRecordingNumerationStyle::AppendFrameNumber)
		, ImageFormat(EMediaPlayerRecordingImageFormat::BMP)
		, CompressionQuality(0)
		, bResetAlpha(false)
	{

	}

	/** Whether this MediaPlayer is active and its event will be recorded when the 'Record' button is pressed. */
	UPROPERTY(EditAnywhere, Category = "MediaPlayer Event Recording")
	bool bActive;

	/** Whether this MediaPlayer is active and the image frame will be recorded when the 'Record' button is pressed. */
	UPROPERTY(EditAnywhere, Category = "MediaPlayer Frame Recording")
	bool bRecordMediaFrame;

	/** How to name each frame. */
	UPROPERTY(EditAnywhere, Category = "MediaPlayer Frame Recording", meta = (EditCondition = "bRecordMediaFrame"))
	FString BaseFilename;

	/** How to numerate the filename. */
	UPROPERTY(EditAnywhere, Category = "MediaPlayer Frame Recording", meta = (EditCondition = "bRecordMediaFrame"))
	EMediaPlayerRecordingNumerationStyle NumerationStyle;

	/** The image format we wish to record to. */
	UPROPERTY(EditAnywhere, Category = "MediaPlayer Frame Recording", meta = (EditCondition = "bRecordMediaFrame"))
	EMediaPlayerRecordingImageFormat ImageFormat;

	/**
	 * An image format specific compression setting.
	 * For EXRs, either 0 (Default) or 1 (Uncompressed)
	 * For PNGs & JPEGs, 0 (Default) or a value between 1 (worst quality, best compression) and 100 (best quality, worst compression)
	 */
	UPROPERTY(EditAnywhere, Category = "MediaPlayer Frame Recording", meta = (EditCondition = "bRecordMediaFrame"))
	int32 CompressionQuality;

	/**
	 * If the format support it, set the alpha to 1 (or 255).
	 * @note Removing alpha increase the memory foot print.
	 */
	UPROPERTY(EditAnywhere, Category = "MediaPlayer Frame Recording", meta = (EditCondition = "bRecordMediaFrame"))
	bool bResetAlpha;
};

UCLASS(MinimalAPI)
class UMediaPlayerRecording : public USequenceRecordingBase
{
	GENERATED_UCLASS_BODY()

public:
	/** UItemRecordingBase override */
	virtual bool StartRecording(class ULevelSequence* CurrentSequence = nullptr, float CurrentSequenceTime = 0.0f, const FString& BaseAssetPath = FString(), const FString& SessionName = FString()) override;
	virtual bool StopRecording(class ULevelSequence* CurrentSequence = nullptr, float CurrentSequenceTime = 0.0f) override;
	virtual void Tick(ULevelSequence* CurrentSequence = nullptr, float CurrentSequenceTime = 0.0f) override;
	virtual bool IsRecording() const override;
	virtual UObject* GetObjectToRecord() const override { return GetMediaPlayerToRecord(); }
	virtual bool IsActive() const { return RecordingSettings.bActive; }
	virtual FString GetRecordingLabel() const { return GetMediaPlayerToRecord() ? GetMediaPlayerToRecord()->GetName() : FString(); }
//
	/** Get the MediaPlayer to record. */
	UMediaPlayer* GetMediaPlayerToRecord() const;

	/** Set the MediaPlayer to record */
	void SetMediaPlayerToRecord(UMediaPlayer* InMediaPlayer);

public:
	/** Whether this MediaPlayer is active and its event will be recorded when the 'Record' button is pressed. */
	UPROPERTY(EditAnywhere, Category = "MediaPlayer Recording")
	FMediaPlayerRecordingSettings RecordingSettings;

private:
	/** The MediaPlayer we want to record */
	UPROPERTY(EditAnywhere, Category = "MediaPlayer Recording")
	TWeakObjectPtr<UMediaPlayer> MediaPlayerToRecord;

	/** This MediaPlayer's current set of section recorders */
	TArray<TSharedPtr<class IMovieSceneSectionRecorder>> SectionRecorders;
};
