// Copyright Epic Games, Inc. All Rights Reserved.
#include "MediaSequenceRecorderExtender.h"

#include "DragAndDrop/AssetDragDropOp.h"
#include "ISequenceRecorder.h"
#include "MediaPlayer.h"
#include "MediaPlayerRecording.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISinglePropertyView.h"
#include "SDropTarget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaSequenceRecorderExtender)

#define LOCTEXT_NAMESPACE "MediaSequenceRecorder"

UMediaSequenceRecorderSettings::UMediaSequenceRecorderSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bRecordMediaPlayerEnabled = false;
	MediaPlayerSubDirectory = TEXT("MediaPlayer");
}

void UMediaSequenceRecorderSettings::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	SaveConfig();
}

static const FName ActiveColumnName(TEXT("Active"));
static const FName FrameColumnName(TEXT("Frame"));
static const FName ItemColumnName(TEXT("Item"));

/** A widget to display information about a MediaPlayer recording in the list view */
class SSequenceRecorderMediaPlayerListRow : public SMultiColumnTableRow<UMediaPlayerRecording*>
{
public:
	SLATE_BEGIN_ARGS(SSequenceRecorderMediaPlayerListRow) {}

	/** The list item for this row */
	SLATE_ARGUMENT(UMediaPlayerRecording*, Recording)

		SLATE_END_ARGS()

		void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		RecordingPtr = Args._Recording;

		SMultiColumnTableRow<UMediaPlayerRecording*>::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f),
			OwnerTableView
		);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == ActiveColumnName)
		{
			return
				SNew(SButton)
				.ContentPadding(0.0f)
				.OnClicked(this, &SSequenceRecorderMediaPlayerListRow::ToggleRecordingActive)
				.ButtonStyle(FAppStyle::Get(), "NoBorder")
				.ToolTipText(LOCTEXT("ActiveButtonToolTip", "Toggle Recording Active"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Content()
				[
					SNew(SImage)
					.Image(this, &SSequenceRecorderMediaPlayerListRow::GetEventBrushForRecording)
				];
		}
		else if (ColumnName == FrameColumnName)
		{
			return
				SNew(SButton)
				.ContentPadding(0.0f)
				.OnClicked(this, &SSequenceRecorderMediaPlayerListRow::ToggleRecordingFrame)
				.ButtonStyle(FAppStyle::Get(), "NoBorder")
				.ToolTipText(LOCTEXT("VideoFramesButtonToolTip", "Toggle Recording Video Frames\nSelect Media Player to show Media Player Recording Options"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Content()
				[
					SNew(SImage)
					.Image(this, &SSequenceRecorderMediaPlayerListRow::GetFrameBrushForRecording)
				];
		}
		else if (ColumnName == ItemColumnName)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(2.0f, 0.0f, 2.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SSequenceRecorderMediaPlayerListRow::GetRecordingMediaPlayerName)))
				];
		}

		return SNullWidget::NullWidget;
	}

private:
	FReply ToggleRecordingActive()
	{
		if (RecordingPtr.IsValid())
		{
			RecordingPtr.Get()->RecordingSettings.bActive = !RecordingPtr.Get()->RecordingSettings.bActive;
		}
		return FReply::Handled();
	}

	const FSlateBrush* GetEventBrushForRecording() const
	{
		if (RecordingPtr.IsValid() && RecordingPtr.Get()->RecordingSettings.bActive)
		{
			return FAppStyle::GetBrush("SequenceRecorder.Common.RecordingActive");
		}
		else
		{
			return FAppStyle::GetBrush("SequenceRecorder.Common.RecordingInactive");
		}
	}

	FReply ToggleRecordingFrame()
	{
		if (RecordingPtr.IsValid())
		{
			RecordingPtr.Get()->RecordingSettings.bRecordMediaFrame = !RecordingPtr.Get()->RecordingSettings.bRecordMediaFrame;
		}
		return FReply::Handled();
	}

	const FSlateBrush* GetFrameBrushForRecording() const
	{
		if (RecordingPtr.IsValid() && RecordingPtr.Get()->RecordingSettings.bRecordMediaFrame)
		{
			return FAppStyle::GetBrush("SequenceRecorder.Common.RecordingActive");
		}
		else
		{
			return FAppStyle::GetBrush("SequenceRecorder.Common.RecordingInactive");
		}
	}

	FText GetRecordingMediaPlayerName() const
	{
		FText ItemName(LOCTEXT("InvalidActorName", "None"));
		if (RecordingPtr.IsValid() && RecordingPtr.Get()->GetMediaPlayerToRecord() != nullptr)
		{
			ItemName = FText::FromName(RecordingPtr.Get()->GetMediaPlayerToRecord()->GetFName());
		}

		return ItemName;
	}

	TWeakObjectPtr<UMediaPlayerRecording> RecordingPtr;
};

FMediaSequenceRecorderExtender::FMediaSequenceRecorderExtender()
	: bInsideSelectionChanged(false)
{
}

TSharedPtr<IDetailsView> FMediaSequenceRecorderExtender::MakeSettingDetailsView()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowSearch = false;

	TSharedPtr<IDetailsView> SettingDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	SettingDetailsView->SetObject(GetMutableDefault<UMediaSequenceRecorderSettings>());

	return SettingDetailsView;
}

TSharedPtr<SWidget> FMediaSequenceRecorderExtender::MakeListWidget(TSharedPtr<SListView<USequenceRecordingBase*>>& OutCreatedListView, FListViewSelectionChanged InOnListViewSelectionChanged)
{
	OnListViewSelectionChanged = InOnListViewSelectionChanged;

	TSharedPtr<SBox> Result;
	SAssignNew(Result, SBox)
		.Visibility(this, &FMediaSequenceRecorderExtender::GetRecordMediaPlayerVisible)
		[
			SNew(SDropTarget)
			.OnAllowDrop(this, &FMediaSequenceRecorderExtender::OnRecordingMediaPlayerListAllowDrop)
			.OnDropped(this, &FMediaSequenceRecorderExtender::OnRecordingMediaPlayerListDrop)
			.Content()
			[
				SAssignNew(MediaPlayerListView, SListView<USequenceRecordingBase*>)
				.ListItemsSource(&QueuedMediaPlayerRecordings)
				.SelectionMode(ESelectionMode::SingleToggle)
				.OnGenerateRow(this, &FMediaSequenceRecorderExtender::MakeMediaPlayerListViewWidget)
				.OnSelectionChanged(this, &FMediaSequenceRecorderExtender::OnMediaPlayerListSelectionChanged)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column(ActiveColumnName)
					.FixedWidth(50.0f)
					.DefaultLabel(LOCTEXT("ActiveColumnName", "Active"))

					+ SHeaderRow::Column(ItemColumnName)
					.FillWidth(30.0f)
					.DefaultLabel(LOCTEXT("MediaHeaderName", "MediaPlayer"))

					+ SHeaderRow::Column(FrameColumnName)
					.FixedWidth(50.f)
					.DefaultLabel(LOCTEXT("FrameColumnName", "Frames"))
				)
			]
		];

	OutCreatedListView = MediaPlayerListView;
	return StaticCastSharedPtr<SWidget>(Result);
}

void FMediaSequenceRecorderExtender::SetListViewSelection(USequenceRecordingBase* InSelectedBase)
{
	if (bInsideSelectionChanged)
	{
		return;
	}

	if (MediaPlayerListView.IsValid())
	{
		TGuardValue<bool> TmpValue(bInsideSelectionChanged, true);
		if (UMediaPlayerRecording* MediaPlayerRecording = Cast<UMediaPlayerRecording>(InSelectedBase))
		{
			MediaPlayerListView->SetSelection(MediaPlayerRecording, ESelectInfo::Direct);
		}
		else
		{
			MediaPlayerListView->SetSelection(nullptr, ESelectInfo::Direct);
		}
	}
}

USequenceRecordingBase* FMediaSequenceRecorderExtender::AddNewQueueRecording(UObject* SequenceRecordingObjectToRecord)
{
	if (!GetDefault<UMediaSequenceRecorderSettings>()->bRecordMediaPlayerEnabled)
	{
		return nullptr;
	}

	UMediaPlayer* MediaPlayer = Cast<UMediaPlayer>(SequenceRecordingObjectToRecord);
	if (MediaPlayer == nullptr)
	{
		return nullptr;
	}

	if (FindRecording(MediaPlayer))
	{
		return nullptr;
	}

	UMediaPlayerRecording* MediaRecording = NewObject<UMediaPlayerRecording>(GetTransientPackage());
	MediaRecording->AddToRoot();
	MediaRecording->SetMediaPlayerToRecord(MediaPlayer);

	QueuedMediaPlayerRecordings.Add(MediaRecording);

	return MediaRecording;
}

void FMediaSequenceRecorderExtender::BuildQueuedRecordings(const TArray<USequenceRecordingBase*>& InQueuedRecordings)
{
	QueuedMediaPlayerRecordings.Reset();
	for (USequenceRecordingBase* QueuedRecording : InQueuedRecordings)
	{
		if (UMediaPlayerRecording* MediaRecording = Cast<UMediaPlayerRecording>(QueuedRecording))
		{
			QueuedMediaPlayerRecordings.Add(MediaRecording);
		}
	}
}

UMediaPlayerRecording* FMediaSequenceRecorderExtender::FindRecording(UMediaPlayer* InMediaPlayer) const
{
	for (USequenceRecordingBase* QueuedRecording : QueuedMediaPlayerRecordings)
	{
		if (QueuedRecording->GetObjectToRecord() == InMediaPlayer)
		{
			return CastChecked<UMediaPlayerRecording>(QueuedRecording);
		}
	}

	return nullptr;
}

TSharedRef<ITableRow> FMediaSequenceRecorderExtender::MakeMediaPlayerListViewWidget(USequenceRecordingBase* InRecording, const TSharedRef<STableViewBase>& OwnerTable) const
{
	UMediaPlayerRecording* MediaPlayerRecording = CastChecked<UMediaPlayerRecording>(InRecording);
	return SNew(SSequenceRecorderMediaPlayerListRow, OwnerTable)
		.Recording(MediaPlayerRecording);
}

void FMediaSequenceRecorderExtender::OnMediaPlayerListSelectionChanged(USequenceRecordingBase* InRecording, ESelectInfo::Type SelectionType)
{
	if (bInsideSelectionChanged)
	{
		return;
	}

	TGuardValue<bool> TmpValue(bInsideSelectionChanged, true);
	if (OnListViewSelectionChanged.IsBound())
	{
		OnListViewSelectionChanged.Execute(InRecording);
	}
}

EVisibility FMediaSequenceRecorderExtender::GetRecordMediaPlayerVisible() const
{
	const UMediaSequenceRecorderSettings* Settings = GetDefault<UMediaSequenceRecorderSettings>();
	return Settings->bRecordMediaPlayerEnabled ? EVisibility::Visible : EVisibility::Collapsed;
}

bool FMediaSequenceRecorderExtender::OnRecordingMediaPlayerListAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation)
{
	bool bResult = false;
	if (DragDropOperation->IsOfType<FAssetDragDropOp>())
	{
		TSharedPtr<FAssetDragDropOp> AssetDragDropOperation = StaticCastSharedPtr<FAssetDragDropOp>(DragDropOperation);
		bResult = true;
		for (const FAssetData& AssetData : AssetDragDropOperation->GetAssets())
		{
			if (!AssetData.IsValid() || !AssetData.IsInstanceOf(UMediaPlayer::StaticClass()))
			{
				bResult = false;
				break;
			}
		}
	}
	return bResult;
}


 FReply FMediaSequenceRecorderExtender::OnRecordingMediaPlayerListDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<FAssetDragDropOp> AssetDragDropOperation = InDragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (AssetDragDropOperation)
	{
		for (const FAssetData& AssetData : AssetDragDropOperation->GetAssets())
		{
			if (AssetData.IsValid() && AssetData.IsInstanceOf(UMediaPlayer::StaticClass()))
			{
				UMediaPlayer* MediaPlayer = Cast<UMediaPlayer>(AssetData.GetAsset());
				if (MediaPlayer)
				{
					ISequenceRecorder* Recorder = FModuleManager::Get().GetModulePtr<ISequenceRecorder>("SequenceRecorder");
					if (Recorder)
					{
						Recorder->QueueObjectToRecord(MediaPlayer);
					}
				}
			}
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE

