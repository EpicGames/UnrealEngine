// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISequenceRecorderExtender.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Sequencer/MediaPlayerRecording.h"
#include "Types/SlateEnums.h"
#include "Widgets/Views/SListView.h"

#include "MediaSequenceRecorderExtender.generated.h"

class FDragDropOperation;
class ITableRow;
class UMediaPlayer;

UCLASS(config = Editor)
class UMediaSequenceRecorderSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
public:

	/** Whether to enabled MediaPlayer recording into this sequence. */
	UPROPERTY(Config, EditAnywhere, Category = "MediaPlayer Recording")
	bool bRecordMediaPlayerEnabled;

	/** The name of the subdirectory MediaPlayer will be placed in. Leave this empty to place into the same directory as the sequence base path */
	UPROPERTY(Config, EditAnywhere, Category = "MediaPlayer Recording")
	FString MediaPlayerSubDirectory;
};

/**
 * Sequence Recorder extender to record media player
 */
class FMediaSequenceRecorderExtender : public ISequenceRecorderExtender
{
public:
	FMediaSequenceRecorderExtender();

public:
	virtual TSharedPtr<IDetailsView> MakeSettingDetailsView() override;
	virtual TSharedPtr<SWidget> MakeListWidget(TSharedPtr<SListView<USequenceRecordingBase*>>& OutCreatedListView, FListViewSelectionChanged OnListViewSelectionChanged) override;
	virtual void SetListViewSelection(USequenceRecordingBase* InSelectedBase) override;
	virtual USequenceRecordingBase* AddNewQueueRecording(UObject* SequenceRecordingObjectToRecord) override;
	virtual void BuildQueuedRecordings(const TArray<USequenceRecordingBase*>& InQueuedRecordings) override;

private:
	UMediaPlayerRecording * FindRecording(UMediaPlayer* InMediaPlayer) const;

	TSharedRef<ITableRow> MakeMediaPlayerListViewWidget(USequenceRecordingBase* Recording, const TSharedRef<STableViewBase>& OwnerTable) const;
	void OnMediaPlayerListSelectionChanged(USequenceRecordingBase* Recording, ESelectInfo::Type SelectionType);
	EVisibility GetRecordMediaPlayerVisible() const;
	bool OnRecordingMediaPlayerListAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation);
	FReply OnRecordingMediaPlayerListDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);

private:
	bool bInsideSelectionChanged;
	FListViewSelectionChanged OnListViewSelectionChanged;
	TSharedPtr<SListView<USequenceRecordingBase*>> MediaPlayerListView;
	TArray<USequenceRecordingBase*> QueuedMediaPlayerRecordings;
};
