// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaSequencerFilters.h"
#include "MovieSceneMediaTrack.h"
#include "Filters/SequencerTrackFilterBase.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaSequencerFilters)

#define LOCTEXT_NAMESPACE "MediaSequencerTrackFilters"

class FSequencerTrackFilter_MediaFilterCommands
	: public TCommands<FSequencerTrackFilter_MediaFilterCommands>
{
public:
	FSequencerTrackFilter_MediaFilterCommands()
		: TCommands<FSequencerTrackFilter_MediaFilterCommands>(
			TEXT("FSequencerTrackFilter_Media"),
			LOCTEXT("FSequencerTrackFilter_Media", "Media Filters"),
			NAME_None,
			FAppStyle::GetAppStyleSetName())
	{}

	TSharedPtr<FUICommandInfo> ToggleFilter_Media;

	virtual void RegisterCommands() override
	{
		UI_COMMAND(ToggleFilter_Media, "Toggle Media Filter", "Toggle the filter for Media tracks", EUserInterfaceActionType::ToggleButton, FInputChord());
	}
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_Media : public FSequencerTrackFilter_ClassType<UMovieSceneMediaTrack>
{
public:
	FSequencerTrackFilter_Media(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr)
		: FSequencerTrackFilter_ClassType<UMovieSceneMediaTrack>(InFilterInterface, InCategory)
		, BindingCount(0)
	{
		FSequencerTrackFilter_MediaFilterCommands::Register();
	}

	virtual ~FSequencerTrackFilter_Media() override
	{
		BindingCount--;
		if (BindingCount < 1)
		{
			FSequencerTrackFilter_MediaFilterCommands::Unregister();
		}
	}

	//~ Begin IFilter
	virtual FString GetName() const override { return TEXT("Media"); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override { return LOCTEXT("SequencerTrackFilter_Media", "Media"); }
	virtual FSlateIcon GetIcon() const override { return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.Tracks.Media")); }
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter

	virtual FText GetDefaultToolTipText() const override
	{
		return LOCTEXT("SequencerTrackFilter_MediaToolTip", "Show only Media tracks");
	}

	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override
	{
		return FSequencerTrackFilter_MediaFilterCommands::Get().ToggleFilter_Media;
	}

	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override
	{
		return IsSequenceTrackSupported<UMovieSceneMediaTrack>(InSequence);
	}

	//~ End FSequencerTrackFilter

private:
	mutable uint32 BindingCount;
};

//////////////////////////////////////////////////////////////////////////
//

void UMediaCompositingTrackFilter::AddTrackFilterExtensions(ISequencerTrackFilters& InFilterInterface, const TSharedRef<FFilterCategory>& InPreferredCategory, TArray<TSharedRef<FSequencerTrackFilter>>& InOutFilterList) const
{
	InOutFilterList.Add(MakeShared<FSequencerTrackFilter_Media>(InFilterInterface, InPreferredCategory));
}

#undef LOCTEXT_NAMESPACE
