// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerTrackFilterExtension.h"
#include "MediaSequencerFilters.generated.h"

UCLASS()
class UMediaCompositingTrackFilter : public USequencerTrackFilterExtension
{
public:
	GENERATED_BODY()

	//~ Begin USequencerTrackFilterExtension
	virtual void AddTrackFilterExtensions(ISequencerTrackFilters& InFilterInterface
		, const TSharedRef<FFilterCategory>& InPreferredCategory
		, TArray<TSharedRef<FSequencerTrackFilter>>& InOutFilterList) const override;
	//~ End USequencerTrackFilterExtension
};
