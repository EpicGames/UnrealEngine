// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MediaThumbnailSection.h"

#include "CommonRenderResources.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Engine/Texture2D.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "IMediaCache.h"
#include "IMediaTracks.h"
#include "ISequencer.h"
#include "ImageCoreUtils.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "MediaSource.h"
#include "Misc/ObjectThumbnail.inl"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneMediaData.h"
#include "MovieSceneMediaSection.h"
#include "MovieSceneTimeHelpers.h"
#include "ObjectTools.h"
#include "RHI.h"
#include "RHIStaticStates.h"
#include "RendererInterface.h"
#include "Rendering/DrawElements.h"
#include "ScreenRendering.h"
#include "SequencerSectionPainter.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "TimeToPixel.h"
#include "TrackEditorThumbnail/TrackEditorThumbnailPool.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FMediaThumbnailSection"

namespace UE::MediaThumbnailSection
{
	constexpr float FilmBorderHeight = 9.0f;
	static const FLazyName SectionTitleFont("NormalFont");
	static const FLazyName MediaInfoFont("SmallFont");

	enum class EOffsetOrigin : int8
	{
		FromTop,
		FromBottom
	};

	/** Helper to paint text for media info. */
	struct FTextPaintHelper
	{
		FTextPaintHelper(const FMargin& InContentPadding, const ESlateDrawEffect InDrawEffects, const FSlateFontInfo& InFontInfo)
			: ContentPadding(InContentPadding)
			, DrawEffects(InDrawEffects)
			, FontInfo(InFontInfo)
			, FontMeasureService(FSlateApplication::Get().GetRenderer()->GetFontMeasureService())
		{}

		/**
		 * Paint given string
		 * @param InPainter Painter
		 * @param InString String
		 * @param InVerticalOffset Vertical offset relative to either the top or bottom of the section.
		 * @param InOffsetOrigin Origin of the vertical offset
		 * @param InHorizontalOffset Additional horizontal offset added to the base X anchor. Use this to chain
		 *        multiple PaintString calls on the same line: pass the running sum of prior segments' widths.
		 * @return Size of the painted text in local coordinates.
		 */
		FVector2f PaintString(FSequencerSectionPainter& InPainter, const FString& InString, float InVerticalOffset, EOffsetOrigin InOffsetOrigin, float InHorizontalOffset = 0.0f)
		{
			FColor CurrentColor = TextColor;
			CurrentColor.A = static_cast<uint8>(InPainter.GhostAlpha * 255);
			FVector2f TextSize = FontMeasureService->Measure(InString, FontInfo);
			FVector2D TextOffset(EForceInit::ForceInitToZero);

			// Have the text on the bottom left side of the clip rect, along with single thumbnail and section title.
			if (InOffsetOrigin == EOffsetOrigin::FromBottom)
			{
				FVector2D BottomLeft = InPainter.SectionGeometry.AbsoluteToLocal(InPainter.SectionClippingRect.GetBottomLeft());
				TextOffset.Set(BottomLeft.X + ContentPadding.Left + 2 + InHorizontalOffset, InPainter.SectionGeometry.Size.Y - (TextSize.Y + ContentPadding.Bottom));
				TextOffset.Y += InVerticalOffset;
			}
			else
			{
				FVector2D TopLeft = InPainter.SectionGeometry.AbsoluteToLocal(InPainter.SectionClippingRect.GetTopLeft());
				TextOffset.Set(TopLeft.X + ContentPadding.Left + 2 + InHorizontalOffset, 0);
				TextOffset.Y += InVerticalOffset;
			}
			
			int32 LayerId = InPainter.LayerId++;

			// Drop shadow
			FSlateDrawElement::MakeText(
				InPainter.DrawElements,
				LayerId,
				InPainter.SectionGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextOffset + FVector2D(1.f, 1.f))),
				InString,
				FontInfo,
				DrawEffects,
				FLinearColor(0, 0, 0, .5f * InPainter.GhostAlpha)
			);
			
			FSlateDrawElement::MakeText(
				InPainter.DrawElements,
				LayerId,
				InPainter.SectionGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextOffset)),
				InString,
				FontInfo,
				DrawEffects,
				CurrentColor
			);

			return TextSize;	
		}

		FMargin ContentPadding;
		const ESlateDrawEffect DrawEffects;
		const FSlateFontInfo FontInfo;
		const TSharedRef<FSlateFontMeasure> FontMeasureService;

		FColor TextColor = FColor(192, 192, 192);
	};

	/** Returns the thumbnail for the specified object or NULL if one doesn't exist yet. */
	FObjectThumbnail* GetOrLoadThumbnailForObject(UObject* InObject)
	{
		FObjectThumbnail* ObjectThumbnail = ThumbnailTools::GetThumbnailForObject(InObject);
		if (ObjectThumbnail == nullptr)
		{
			UPackage* ObjectPackage = InObject->GetOutermost();

			// Replicating some logic from ThumbnailManager
			const bool bMemoryPackage = FPackageName::IsMemoryPackage(InObject->GetPathName()); // Don't try to load from disk if the package is a memory package
			const bool bUnsavedPackage = ObjectPackage->HasAnyPackageFlags(PKG_NewlyCreated);   // Don't try loading thumbnails for package that have never been saved

			const bool bTryLoadThumbnailFromDisk = !bMemoryPackage && !bUnsavedPackage;
			if (bTryLoadThumbnailFromDisk)
			{
				FName ObjectFullName = FName(*InObject->GetFullName());

				FThumbnailMap LoadedThumbnails;
				if (ThumbnailTools::ConditionallyLoadThumbnailsForObjects({ObjectFullName}, LoadedThumbnails))
				{
					ObjectThumbnail = LoadedThumbnails.Find(ObjectFullName);

					// If we found one, cache it so that we don't have to do this again
					if (ObjectThumbnail != nullptr)
					{
						ObjectThumbnail = ThumbnailTools::CacheThumbnail(ObjectFullName.ToString(), ObjectThumbnail, ObjectPackage);
					}
				}
			}
		}
		return ObjectThumbnail;
	}

	/** Utility function to try to make the media source's thumbnail from the existing asset thumbnail, if any. */
	const UTexture* TryMakeThumbnailFromAssetThumbnail(UMediaSource* InMediaSource)
	{
		UTexture2D* Texture = nullptr;
		
		const FObjectThumbnail* AssetThumbnail = GetOrLoadThumbnailForObject(InMediaSource);
	
		// Create texture from the asset thumbnail
		if (AssetThumbnail && !AssetThumbnail->IsEmpty())
		{
			const TArray<uint8>& ImageData = AssetThumbnail->GetUncompressedImageData();
			if (ImageData.Num() > 0)
			{
				Texture = UTexture2D::CreateTransient(
					AssetThumbnail->GetImageWidth(),
					AssetThumbnail->GetImageHeight(),
					FImageCoreUtils::GetPixelFormatForRawImageFormat(AssetThumbnail->GetImage().Format),
					NAME_None,
					ImageData);
			}

			if (Texture)
			{
				InMediaSource->SetThumbnail(Texture);	// Store the texture directly in the media source to speed things up.
			}
		}

		if (!AssetThumbnail || !Texture)
		{
			// Cache an empty thumbnail to speed up the process next time.
			ThumbnailTools::CacheEmptyThumbnail(InMediaSource->GetFullName(), InMediaSource->GetPackage());
		}
		return Texture;
	}
}

/* FMediaThumbnailSection structors
 *****************************************************************************/

FMediaThumbnailSection::FMediaThumbnailSection(UMovieSceneMediaSection& InSection, TSharedPtr<FTrackEditorThumbnailPool> InThumbnailPool, TSharedPtr<ISequencer> InSequencer)
	: FThumbnailSection(InSequencer, InThumbnailPool, this, InSection)
	, SectionPtr(&InSection)
{
	TimeSpace = ETimeSpace::Local;

	if (InSequencer)
	{
		InSequencer->OnBeginScrubbingEvent().AddRaw(this, &FMediaThumbnailSection::OnBeginScrubbingEvent);
		InSequencer->OnEndScrubbingEvent().AddRaw(this, &FMediaThumbnailSection::OnEndScrubbingEvent);
	}

	// Build the media source's thumbnail
	if (UMediaSource* MediaSource = GetMediaSource())
	{
		const UTexture* Thumbnail = MediaSource->GetThumbnail();
		if (!Thumbnail)
		{
			UE::MediaThumbnailSection::TryMakeThumbnailFromAssetThumbnail(MediaSource);
		}
	}
}


FMediaThumbnailSection::~FMediaThumbnailSection()
{
	if (const TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin())
	{
		Sequencer->OnBeginScrubbingEvent().RemoveAll(this);
		Sequencer->OnEndScrubbingEvent().RemoveAll(this);
	}
}


/* FGCObject interface
 *****************************************************************************/

void FMediaThumbnailSection::AddReferencedObjects(FReferenceCollector& Collector)
{
}


/* FThumbnailSection interface
 *****************************************************************************/

FMargin FMediaThumbnailSection::GetContentPadding() const
{
	return FMargin(8.0f, 15.0f);
}


float FMediaThumbnailSection::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	using namespace UE::MediaThumbnailSection;

	// Calculate the base section height
	float SectionHeight = FThumbnailSection::GetSectionHeight(ViewDensity);

	// Calculate the section title height.
	const float SectionTitleHeight = FAppStyle::GetFontStyle(SectionTitleFont).Size + 8.f;

	// Calculate minimum space for the section title, media info and optional performance warning.
	const float PlayerInfoHeight = FAppStyle::GetFontStyle(MediaInfoFont).Size + 8.f;
	const float NumInfoLines = CachedWarningString.IsEmpty() ? 1.0f : 2.0f;
	const float MinimumHeight = SectionTitleHeight + PlayerInfoHeight * NumInfoLines + 8.0f;

	// Base section height is either the thumbnail height or just the title.
	// We want to make sure we have enough space for the media info too.
	SectionHeight = FMath::Max(SectionHeight, MinimumHeight);

	// Make additional space for the film border
	return SectionHeight + 2 * FilmBorderHeight;
}

FText FMediaThumbnailSection::GetSectionTitle() const
{
	UMediaSource* MediaSource = GetMediaSource();

	if (MediaSource == nullptr)
	{
		return LOCTEXT("NoSequence", "Empty");
	}

	return FText::FromString(MediaSource->GetFName().ToString());
}


int32 FMediaThumbnailSection::OnPaintSection(FSequencerSectionPainter& InPainter) const
{
	// draw background
	InPainter.LayerId = InPainter.PaintSectionBackground();

	FVector2D SectionSize = InPainter.SectionGeometry.GetLocalSize();
	FSlateClippingZone ClippingZone(InPainter.SectionClippingRect.InsetBy(FMargin(1.0f)));
	
	InPainter.DrawElements.PushClip(ClippingZone);
	{
		DrawFilmBorder(InPainter, SectionSize);
	}
	InPainter.DrawElements.PopClip();

	// draw thumbnails
	int32 LayerId = FThumbnailSection::OnPaintSection(InPainter) + 1;

	UMediaPlayer* MediaPlayer = GetTemplateMediaPlayer();

	if (MediaPlayer == nullptr)
	{
		return LayerId;
	}

	// draw overlays
	const FTimespan MediaDuration = MediaPlayer->GetDuration();

	if (MediaDuration.IsZero())
	{
		return LayerId;
	}

	TSharedRef<FMediaPlayerFacade, ESPMode::ThreadSafe> MediaPlayerFacade = MediaPlayer->GetPlayerFacade();

	InPainter.DrawElements.PushClip(ClippingZone);
	{
		TRangeSet<FTimespan> CacheRangeSet;

		MediaPlayerFacade->QueryCacheState(EMediaTrackType::Video, EMediaCacheState::Pending, CacheRangeSet);
		DrawSampleStates(InPainter, MediaDuration, SectionSize, CacheRangeSet, FLinearColor::Gray);

		CacheRangeSet.Empty();

		MediaPlayerFacade->QueryCacheState(EMediaTrackType::Video, EMediaCacheState::Loading, CacheRangeSet);
		DrawSampleStates(InPainter, MediaDuration, SectionSize, CacheRangeSet, FLinearColor::Yellow);

		CacheRangeSet.Empty();

		MediaPlayerFacade->QueryCacheState(EMediaTrackType::Video, EMediaCacheState::Loaded, CacheRangeSet);
		DrawSampleStates(InPainter, MediaDuration, SectionSize, CacheRangeSet, FLinearColor(0.10616f, 0.48777f, 0.10616f));

		CacheRangeSet.Empty();

		MediaPlayerFacade->QueryCacheState(EMediaTrackType::Video, EMediaCacheState::Cached, CacheRangeSet);
		DrawSampleStates(InPainter, MediaDuration, SectionSize, CacheRangeSet, FLinearColor(0.07059f, 0.32941f, 0.07059f));

		DrawLoopIndicators(InPainter, MediaDuration, SectionSize);

		DrawMediaInfo(InPainter, MediaPlayer, SectionSize);
	}
	InPainter.DrawElements.PopClip();

	return LayerId;
}


void FMediaThumbnailSection::SetSingleTime(double GlobalTime)
{
	UMovieSceneMediaSection* MediaSection = CastChecked<UMovieSceneMediaSection>(Section);

	if (MediaSection != nullptr)
	{
		double StartTime = MediaSection->GetInclusiveStartFrame() / MediaSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		MediaSection->SetThumbnailReferenceOffset(static_cast<float>(GlobalTime - StartTime));
	}
}


void FMediaThumbnailSection::Tick(const FGeometry& AllottedGeometry, const FGeometry& ClippedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	UMovieSceneMediaSection* MediaSection = Cast<UMovieSceneMediaSection>(Section);

	const bool bPreviousPerformanceWarning = bDrawSeekPerformanceWarning;
	const bool bPreviousTextureWarning = bDrawMissingMediaTextureWarning;

	bDrawMissingMediaTextureWarning = false;

	if (MediaSection != nullptr)
	{
		if (GetDefault<UMovieSceneUserThumbnailSettings>()->bDrawSingleThumbnails)
		{
			ThumbnailCache.SetSingleReferenceFrame(MediaSection->GetThumbnailReferenceOffset());
		}
		else
		{
			ThumbnailCache.SetSingleReferenceFrame(TOptional<double>());
		}

		// Logic for updating the "missing media texture" warning message.
		if (!MediaSection->bUseExternalMediaPlayer				// Using external player has its own texture. (we can't check anyway)
			&& !MediaSection->bHasMediaPlayerProxy				// Using a media player proxy will have its own texture.
			&& MediaSection->MediaTexture == nullptr)			// Section texture is unspecified.
		{
			bDrawMissingMediaTextureWarning = true;
		}
	}

	if (const UMediaPlayer* MediaPlayer = GetTemplateMediaPlayer())
	{
		UpdateCachedMediaInfo(MediaPlayer);
	}
	else
	{
		// No player: nothing to compare; clear the inline frame rate state.
		bFrameRateMismatch = false;
		PlayerInfoPrefix.Reset();
		PlayerFrameRateText.Reset();
	}

	if (bPreviousPerformanceWarning != bDrawSeekPerformanceWarning
		|| bPreviousTextureWarning != bDrawMissingMediaTextureWarning
		|| CachedWarningString.IsEmpty())
	{
		TArray<const FText*, TInlineAllocator<2>> Warnings;
		if (bDrawSeekPerformanceWarning)
		{
			static const FText SeekPerformanceWarning = LOCTEXT("SeekPerformance", "Slow Seek Performance (GOP codec)");
			Warnings.Add(&SeekPerformanceWarning);
		}
		if (bDrawMissingMediaTextureWarning)
		{
			static const FText MissingTextureWarning = LOCTEXT("MissingMediaTexture", "Missing Media Texture");
			Warnings.Add(&MissingTextureWarning);
		}
		CachedWarningString = FString::JoinBy(Warnings, TEXT(" - "), [](const FText *InText) {return InText->ToString();});
	}

	FThumbnailSection::Tick(AllottedGeometry, ClippedGeometry, InCurrentTime, InDeltaTime);
}

void FMediaThumbnailSection::BeginResizeSection()
{
	UMovieSceneMediaSection* MediaSection = Cast<UMovieSceneMediaSection>(Section);
	InitialStartOffsetDuringResize = MediaSection->StartFrameOffset;
	InitialStartTimeDuringResize = MediaSection->HasStartFrame() ? MediaSection->GetInclusiveStartFrame() : 0;
}

void FMediaThumbnailSection::ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime)
{
	UMovieSceneMediaSection* MediaSection = Cast<UMovieSceneMediaSection>(Section);

	if (ResizeMode == SSRM_LeadingEdge && MediaSection)
	{
		FFrameNumber StartOffset = ResizeTime - InitialStartTimeDuringResize;
		StartOffset += InitialStartOffsetDuringResize;

		// Ensure start offset is not less than 0
		if (StartOffset < 0)
		{
			ResizeTime = ResizeTime - StartOffset;

			StartOffset = FFrameNumber(0);
		}

		MediaSection->StartFrameOffset = StartOffset;
	}

	ISequencerSection::ResizeSection(ResizeMode, ResizeTime);
}

void FMediaThumbnailSection::BeginSlipSection()
{
	UMovieSceneMediaSection* MediaSection = Cast<UMovieSceneMediaSection>(Section);
	InitialStartOffsetDuringResize = MediaSection->StartFrameOffset;
	InitialStartTimeDuringResize = MediaSection->HasStartFrame() ? MediaSection->GetInclusiveStartFrame() : 0;
}

void FMediaThumbnailSection::SlipSection(FFrameNumber SlipTime)
{
	UMovieSceneMediaSection* MediaSection = Cast<UMovieSceneMediaSection>(Section);

	const FFrameRate FrameRate = MediaSection->GetTypedOuter<UMovieScene>()->GetTickResolution();

	FFrameNumber StartOffset = SlipTime - InitialStartTimeDuringResize;
	StartOffset += InitialStartOffsetDuringResize;

	// Ensure start offset is not less than 0
	if (StartOffset < 0)
	{
		SlipTime = SlipTime - StartOffset;

		StartOffset = FFrameNumber(0);
	}

	MediaSection->StartFrameOffset = StartOffset;

	ISequencerSection::SlipSection(SlipTime);
}

/* ICustomThumbnailClient interface
 *****************************************************************************/

void FMediaThumbnailSection::Draw(FTrackEditorThumbnail& TrackEditorThumbnail)
{
	if (UMediaSource* MediaSource = GetMediaSource())
	{
		// Use media source's thumbnail if available.
		const UTexture* Thumbnail = MediaSource->GetThumbnail();
		if (!Thumbnail)
		{
			// Fallback to the asset thumbnail if there is one.
			Thumbnail = UE::MediaThumbnailSection::TryMakeThumbnailFromAssetThumbnail(MediaSource);
		}

		if (Thumbnail)
		{
			const FTextureReferenceRHIRef SourceTexture = Thumbnail->TextureReference.TextureReferenceRHI;
			if (SourceTexture.IsValid())
			{
				// Limit thumbnail size.
				FIntPoint RTSize(SourceTexture->GetDesc().Extent);
				const int32 SourceMaxSize = RTSize.GetMax();
				constexpr int32 ThumbnailSize = 256;
				if (ThumbnailSize < SourceMaxSize)
				{
					RTSize = ((RTSize * ThumbnailSize) / SourceMaxSize);
				}

				TrackEditorThumbnail.bIgnoreAlpha = true;
				TrackEditorThumbnail.ResizeRenderTarget(RTSize);

				if (FSlateTextureRenderTarget2DResource* RenderTarget = TrackEditorThumbnail.GetRenderTarget())
				{
					CopyTexture(RenderTarget, SourceTexture);
				}
			}
		}
	}
}


void FMediaThumbnailSection::Setup()
{
}


/* FMediaThumbnailSection implementation
 *****************************************************************************/

void FMediaThumbnailSection::DrawFilmBorder(FSequencerSectionPainter& InPainter, FVector2D SectionSize) const
{
	static const FSlateBrush* FilmBorder = FAppStyle::GetBrush("Sequencer.Section.FilmBorder");

	// draw top film border
	FSlateDrawElement::MakeBox(
		InPainter.DrawElements,
		InPainter.LayerId++,
		InPainter.SectionGeometry.ToPaintGeometry(FVector2D(SectionSize.X - 2.0f, 7.0f), FSlateLayoutTransform(FVector2D(1.0f, 4.0f))),
		FilmBorder,
		InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect
	);

	// draw bottom film border
	FSlateDrawElement::MakeBox(
		InPainter.DrawElements,
		InPainter.LayerId++,
		InPainter.SectionGeometry.ToPaintGeometry(FVector2D(SectionSize.X - 2.0f, 7.0f), FSlateLayoutTransform(FVector2D(1.0f, SectionSize.Y - 11.0f))),
		FilmBorder,
		InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect
	);
}


void FMediaThumbnailSection::DrawLoopIndicators(FSequencerSectionPainter& InPainter, FTimespan MediaDuration, FVector2D SectionSize) const
{
	using namespace UE::Sequencer;

	static const FSlateBrush* GenericBrush = FCoreStyle::Get().GetBrush("GenericWhiteBox");

	UMovieSceneMediaSection* MediaSection = Cast<UMovieSceneMediaSection>(Section);

	const FTimeToPixel& TimeToPixelConverter = InPainter.GetTimeConverter();

	FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
	double SectionDuration = FFrameTime(UE::MovieScene::DiscreteSize(Section->GetRange())) / TickResolution;
	const float MediaSizeX = static_cast<float>(MediaDuration.GetTotalSeconds() * SectionSize.X / SectionDuration);
	const FFrameNumber SectionOffset = MediaSection->GetRange().HasLowerBound() ? MediaSection->GetRange().GetLowerBoundValue() : 0;
	float DrawOffset = MediaSizeX - TimeToPixelConverter.SecondsToPixel(TickResolution.AsSeconds(SectionOffset + MediaSection->StartFrameOffset));

	while (DrawOffset < SectionSize.X)
	{
		FSlateDrawElement::MakeBox(
			InPainter.DrawElements,
			InPainter.LayerId++,
			InPainter.SectionGeometry.ToPaintGeometry(FVector2D(1.0f, SectionSize.Y), FSlateLayoutTransform(FVector2D(DrawOffset, 0.0f))),
			GenericBrush,
			ESlateDrawEffect::None,
			FLinearColor::Gray
		);

		DrawOffset += MediaSizeX;
	}
}


void FMediaThumbnailSection::DrawSampleStates(FSequencerSectionPainter& InPainter, FTimespan MediaDuration, FVector2D SectionSize, const TRangeSet<FTimespan>& RangeSet, const FLinearColor& Color) const
{
	using namespace UE::Sequencer;

	static const FSlateBrush* GenericBrush = FCoreStyle::Get().GetBrush("GenericWhiteBox");

	UMovieSceneMediaSection* MediaSection = Cast<UMovieSceneMediaSection>(Section);

	const FTimeToPixel& TimeToPixelConverter = InPainter.GetTimeConverter();

	FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
	double SectionDuration = FFrameTime(UE::MovieScene::DiscreteSize(Section->GetRange())) / TickResolution;
	const float MediaSizeX = static_cast<float>(MediaDuration.GetTotalSeconds() * SectionSize.X / SectionDuration);

	TArray<TRange<FTimespan>> Ranges;
	RangeSet.GetRanges(Ranges);
	float LoopDrawOffset = -TimeToPixelConverter.SecondsDeltaToPixel(TickResolution.AsSeconds(MediaSection->StartFrameOffset));

	while (LoopDrawOffset < SectionSize.X)
	{
		for (auto& Range : Ranges)
		{
			const float DrawOffset = static_cast<float>(FMath::RoundToNegativeInfinity(FTimespan::Ratio(Range.GetLowerBoundValue(), MediaDuration) * MediaSizeX) +	
				LoopDrawOffset);
			const float DrawSize = static_cast<float>(FMath::RoundToPositiveInfinity(FTimespan::Ratio(Range.Size<FTimespan>(), MediaDuration) * MediaSizeX));
			const float BarHeight = 4.0f;

			FSlateDrawElement::MakeBox(
				InPainter.DrawElements,
				InPainter.LayerId++,
				InPainter.SectionGeometry.ToPaintGeometry(FVector2D(DrawSize, BarHeight), FSlateLayoutTransform(FVector2D(DrawOffset, SectionSize.Y - BarHeight - 1.0f))),
				GenericBrush,
				ESlateDrawEffect::None,
				Color
			);
		}

		LoopDrawOffset += MediaSizeX;
	}
}

void FMediaThumbnailSection::DrawMediaInfo(FSequencerSectionPainter& InPainter,
	UMediaPlayer* MediaPlayer, FVector2D SectionSize) const
{
	using namespace UE::MediaThumbnailSection;

	FTextPaintHelper TextPaintHelper(
		GetContentPadding(),
		InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
		FAppStyle::GetFontStyle(MediaInfoFont));

	// We have 2 lines of text to render, top one is the player info
	// that we want to render below the section title.
	// Second line is the warning message that we want to render at the bottom of the section.

	// The available height may be less than what we need, some derived classes override the section height.
	const float AvailableSectionHeight = InPainter.SectionGeometry.Size.Y - FilmBorderHeight;

	float FontHeight = TextPaintHelper.FontInfo.Size + 4.f;

	// Calculate the offset from the top for the player info.
	float TextYOffsetFromTop = FilmBorderHeight + 4.f;

	// Check if there is a section title (there may not be)
	if (!GetSectionTitle().IsEmpty())
	{
		TextYOffsetFromTop += FAppStyle::GetFontStyle(SectionTitleFont).Size + 8.f;
	}

	// Ensure we have enough room to render
	if (AvailableSectionHeight >= TextYOffsetFromTop + FontHeight)
	{
		// The player info row is up to two inline segments on the same line:
		//   1. PlayerInfoPrefix (everything before the FPS clause) in default color.
		//   2. PlayerFrameRateText ("24 fps", with " (Sequence: 30 fps)" already appended on mismatch) in the
		//      engine warning color when bFrameRateMismatch is true, default otherwise.
		float HorizontalOffset = 0.0f;
		float MaxSegmentHeight = 0.0f;

		if (!PlayerInfoPrefix.IsEmpty())
		{
			const FString PrefixToPaint = PlayerFrameRateText.IsEmpty() ? PlayerInfoPrefix : (PlayerInfoPrefix + TEXT(", "));
			FVector2f Size = TextPaintHelper.PaintString(InPainter, PrefixToPaint, TextYOffsetFromTop, EOffsetOrigin::FromTop, HorizontalOffset);
			HorizontalOffset += Size.X;
			MaxSegmentHeight = FMath::Max(MaxSegmentHeight, Size.Y);
		}

		const FColor WarningColor = FStyleColors::Warning.GetSpecifiedColor().ToFColor(true);

		if (!PlayerFrameRateText.IsEmpty())
		{
			TGuardValue PushColor(TextPaintHelper.TextColor, bFrameRateMismatch ? WarningColor : TextPaintHelper.TextColor);
			FVector2f Size = TextPaintHelper.PaintString(InPainter, PlayerFrameRateText, TextYOffsetFromTop, EOffsetOrigin::FromTop, HorizontalOffset);
			MaxSegmentHeight = FMath::Max(MaxSegmentHeight, Size.Y);
		}

		// If nothing was painted above, fall back to font height so the warning-row check below behaves as before.
		if (MaxSegmentHeight <= 0.0f)
		{
			MaxSegmentHeight = FontHeight;
		}

		float RemainingSize = AvailableSectionHeight - (TextYOffsetFromTop + MaxSegmentHeight);

		// Avoid having both messages overlapping.
		if (!CachedWarningString.IsEmpty() && RemainingSize > FontHeight)
		{
			TGuardValue PushColor(TextPaintHelper.TextColor, WarningColor);
			TextPaintHelper.PaintString(InPainter, CachedWarningString, 0.0f, EOffsetOrigin::FromBottom);
		}
	}
}

UMediaSource* FMediaThumbnailSection::GetMediaSource() const
{
	UMovieSceneMediaSection* MediaSection = CastChecked<UMovieSceneMediaSection>(Section);
	TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin();
	UMediaSource* MediaSource = nullptr;

	if (MediaSection && Sequencer.IsValid())
	{
		MediaSource = MediaSection->GetMediaSourceOrProxy(*Sequencer, Sequencer->GetFocusedTemplateID());
	}

	return MediaSource;
}

UMediaPlayer* FMediaThumbnailSection::GetTemplateMediaPlayer() const
{
	// locate the track that evaluates this section
	if (!SectionPtr.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin();

	if (!Sequencer.IsValid())
	{
		return nullptr; // no movie scene player
	}

	// @todo: arodham: Test this and/or check dirty/compile?
	FMovieSceneRootEvaluationTemplateInstance& Instance = Sequencer->GetEvaluationTemplate();

	FMovieSceneSequenceID           SequenceId          = Sequencer->GetFocusedTemplateID();
	UMovieSceneCompiledDataManager* CompiledDataManager = Instance.GetCompiledDataManager();
	UMovieSceneSequence*            SubSequence         = Instance.GetSequence(SequenceId);
	FMovieSceneCompiledDataID       CompiledDataID      = CompiledDataManager->GetDataID(SubSequence);

	if (!CompiledDataID.IsValid())
	{
		return nullptr;
	}

	const FMovieSceneEvaluationTemplate* Template = CompiledDataManager->FindTrackTemplate(CompiledDataID);
	if (Template == nullptr)
	{
		return nullptr; // section template not found
	}

	auto OwnerTrack = Cast<UMovieSceneTrack>(SectionPtr->GetOuter());

	if (OwnerTrack == nullptr)
	{
		return nullptr; // media track not found
	}

	const FMovieSceneTrackIdentifier  TrackIdentifier = Template->GetLedger().FindTrackIdentifier(OwnerTrack->GetSignature());
	const FMovieSceneEvaluationTrack* EvaluationTrack = Template->FindTrack(TrackIdentifier);

	if (EvaluationTrack == nullptr)
	{
		return nullptr; // evaluation track not found
	}

	FMovieSceneMediaData* MediaData = nullptr;

	// find the persistent data of the section being drawn
	TArrayView<const FMovieSceneEvalTemplatePtr> Children = EvaluationTrack->GetChildTemplates();
	FPersistentEvaluationData PersistentData(*Sequencer.Get());

	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		if (Children[ChildIndex]->GetSourceSection() == SectionPtr)
		{
			FMovieSceneEvaluationKey SectionKey(SequenceId, TrackIdentifier, ChildIndex);
			PersistentData.SetSectionKey(SectionKey);
			MediaData = PersistentData.FindSectionData<FMovieSceneMediaData>();

			break;
		}
	}

	// get the template's media player
	if (MediaData == nullptr)
	{
		return nullptr; // section persistent data not found
	}

	return MediaData->GetMediaPlayer();
}

void FMediaThumbnailSection::CopyTexture(FSlateTextureRenderTarget2DResource* RenderTarget, FTextureReferenceRHIRef SourceTexture)
{
	ENQUEUE_RENDER_COMMAND(MediaThumbnailCopyTexture)(
		[SourceTexture, RenderTarget](FRHICommandListImmediate& RHICmdList)
		{
			IRendererModule* RendererModule = &FModuleManager::GetModuleChecked<IRendererModule>("Renderer");

			FTextureRHIRef TargetTexture = RenderTarget->GetRenderTargetTexture();
			if (TargetTexture.IsValid())
			{
				RHICmdList.Transition(FRHITransitionInfo(TargetTexture, ERHIAccess::Unknown, ERHIAccess::RTV));

				FRHIRenderPassInfo RPInfo(TargetTexture, ERenderTargetActions::Load_Store);
				RHICmdList.BeginRenderPass(RPInfo, TEXT("MediaThumbnailCopyTexture"));
				{
					RHICmdList.SetViewport(0.0f, 0.0f, 0.0f,
						static_cast<float>(TargetTexture->GetSizeX()),
						static_cast<float>(TargetTexture->GetSizeY()), 1.0f);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

					FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
					TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
					TShaderMapRef<FScreenPS> PixelShader(ShaderMap);

					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

					GraphicsPSOInit.PrimitiveType = PT_TriangleList;

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					const bool bSameSize = (TargetTexture->GetDesc().Extent == SourceTexture->GetDesc().Extent);
					FRHISamplerState* PixelSampler = bSameSize ? TStaticSamplerState<SF_Point>::GetRHI() : TStaticSamplerState<SF_Bilinear>::GetRHI();

					SetShaderParametersLegacyPS(RHICmdList, PixelShader, PixelSampler, SourceTexture);

					RendererModule->DrawRectangle(
						RHICmdList,
						0.0f, 0.0f,					// Dest X, Y
						static_cast<float>(TargetTexture->GetSizeX()),	// Dest Width
						static_cast<float>(TargetTexture->GetSizeY()),	// Dest Height
						0, 0,						// Source U, V
						1, 1,						// Source USize, VSize
						TargetTexture->GetSizeXY(),	// Target buffer size
						FIntPoint(1, 1),			// Source texture size
						VertexShader,
						EDRF_Default);
				}
				RHICmdList.EndRenderPass();
				RHICmdList.Transition(FRHITransitionInfo(TargetTexture, ERHIAccess::RTV, ERHIAccess::SRVMask));
			}
		});
}

void FMediaThumbnailSection::OnBeginScrubbingEvent()
{
	bIsSequencerScrubbing = true;
}

void FMediaThumbnailSection::OnEndScrubbingEvent()
{
	bIsSequencerScrubbing = false;
}

void FMediaThumbnailSection::UpdateCachedMediaInfo(const UMediaPlayer* InMediaPlayer)
{
	// Note: Protron IsPaused() returns false even if Rate is zero
	const bool bPlayerIsPaused = InMediaPlayer->GetRate() == 0.0f; 
	
	// Only check if player is paused
	if (bPlayerIsPaused)
	{
		// Only check for seek performance warning if the sequencer is scrubbing.
		if (bIsSequencerScrubbing)
		{
			int32 KeyframeInterval = -1;
			// Note: This is only supported by Protron currently.
			if (InMediaPlayer->GetMediaInfo<int32>(KeyframeInterval, UMediaPlayer::MediaInfoNameKeyframeInterval.Resolve()))
			{
				// The keyframe interval is one of:
				//  -1 : no information returned (unknown)
				//   0 : unknown keyframe spacing, not every frame is a keyframe but the spacing is variable or cannot be determined
				//   1 : every frame is a keyframe
				//  >1 : every n'th frame is a keyframe
				if (KeyframeInterval != 1)
				{
					bDrawSeekPerformanceWarning = true;
				}
			}
		}
	}
	else
	{
		bDrawSeekPerformanceWarning = false;
	}

	// Reset the cached info segments and the mismatch flag every update; they are repopulated below.
	PlayerInfoPrefix.Reset(512);
	PlayerFrameRateText.Reset(128);
	bFrameRateMismatch = false;

	// Player name, format, and resolution: paused-only to avoid overloading on playback.
	if (bPlayerIsPaused)
	{
		PlayerInfoPrefix = InMediaPlayer->GetPlayerName().ToString();

		constexpr int32 SelectedTrackIndex = INDEX_NONE;
		constexpr int32 SelectedFormatIndex = INDEX_NONE;

		FString Format = InMediaPlayer->GetVideoTrackType(SelectedTrackIndex, SelectedFormatIndex);
		if (!Format.IsEmpty())
		{
			PlayerInfoPrefix.Appendf(TEXT("%s%s"), PlayerInfoPrefix.IsEmpty() ? TEXT("") : TEXT(", "), *Format);
		}

		FIntPoint Resolution = InMediaPlayer->GetVideoTrackDimensions(SelectedTrackIndex, SelectedFormatIndex);
		if (Resolution.X != 0 || Resolution.Y != 0)
		{
			PlayerInfoPrefix.Appendf(TEXT("%s%dx%d"), PlayerInfoPrefix.IsEmpty() ? TEXT("") : TEXT(", "), Resolution.X, Resolution.Y);
		}
	}

	// Tile and mip counts are shown regardless of pause state. They precede the FPS clause so the
	// FPS sits at the end of the line, where the inline sequence-FPS suffix can follow it.
	FIntPoint TileNum(EForceInit::ForceInitToZero);
	if (InMediaPlayer->GetMediaInfo<FIntPoint>(TileNum, UMediaPlayer::MediaInfoNameSourceNumTiles.Resolve()))
	{
		int32 TileTotalNum = TileNum.X * TileNum.Y;
		if (TileTotalNum > 1)
		{
			PlayerInfoPrefix.Appendf(TEXT("%s%s"), PlayerInfoPrefix.IsEmpty() ? TEXT("") : TEXT(", "),
				*FText::Format(LOCTEXT("TileNum", "Tiles: {0}"), TileTotalNum).ToString());
		}
	}

	int32 MipNum = 0;
	if (InMediaPlayer->GetMediaInfo<int32>(MipNum, UMediaPlayer::MediaInfoNameSourceNumMips.Resolve()))
	{
		if (MipNum > 1)
		{
			PlayerInfoPrefix.Appendf(TEXT("%s%s"), PlayerInfoPrefix.IsEmpty() ? TEXT("") : TEXT(", "),
				*FText::Format(LOCTEXT("Mips", "Mips: {0}"), MipNum).ToString());
		}
	}

	// FPS clause and mismatch indicator: paused-only, like the rest of the player FPS info.
	if (bPlayerIsPaused)
	{
		constexpr int32 SelectedTrackIndex = INDEX_NONE;
		constexpr int32 SelectedFormatIndex = INDEX_NONE;
		const float FrameRate = InMediaPlayer->GetVideoTrackFrameRate(SelectedTrackIndex, SelectedFormatIndex);

		UMovieSceneMediaSection* MediaSection = CastChecked<UMovieSceneMediaSection>(Section);

		FText FrameRateInfo;
		if (MediaSection && MediaSection->bManualFrameRateAlignment)
		{
			if (FrameRate > 0.0f)
			{
				FrameRateInfo = FText::Format(LOCTEXT("FrameRateUnits_WithManual", "{0} ({1}) fps"), FrameRate, MediaSection->FrameRateAlignment.AsDecimal());
			}
			else
			{
				FrameRateInfo = FText::Format(LOCTEXT("FrameRateUnits_ManualOnly", "({0}) fps"), MediaSection->FrameRateAlignment.AsDecimal());
			}
		}
		else if (FrameRate > 0.0f)
		{
			FrameRateInfo = FText::Format(LOCTEXT("FrameRateUnits", "{0} fps"), FrameRate);
		}

		if (!FrameRateInfo.IsEmpty())
		{
			PlayerFrameRateText = FrameRateInfo.ToString();
		}

		// Mismatch detection: compare the effective media rate (manual alignment override when set, else native track rate)
		// against the sequence's display rate. Skip when the effective rate is unknown to avoid init-time false positives.
		// On mismatch, append the inline sequence-FPS suffix directly onto PlayerFrameRateText so a single
		// paint call covers the yellow portion of the line.
		if (MediaSection)
		{
			if (const UMovieScene* MovieScene = MediaSection->GetTypedOuter<UMovieScene>())
			{
				const float EffectiveMediaRate = MediaSection->bManualFrameRateAlignment
					? static_cast<float>(MediaSection->FrameRateAlignment.AsDecimal())
					: FrameRate;

				const double DisplayRateDecimal = MovieScene->GetDisplayRate().AsDecimal();

				if (EffectiveMediaRate > 0.0f && DisplayRateDecimal > 0.0
					&& !FMath::IsNearlyEqual(static_cast<double>(EffectiveMediaRate), DisplayRateDecimal, UE_DOUBLE_KINDA_SMALL_NUMBER))
				{
					bFrameRateMismatch = true;
					PlayerFrameRateText.Append(FText::Format(
						LOCTEXT("InlineSequenceFps", " (Sequence: {0} fps)"),
						DisplayRateDecimal).ToString());
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
