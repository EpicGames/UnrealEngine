// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MediaTrackEditor.h"

#include "ContentBrowserModule.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "IMediaAssetsModule.h"
#include "ISequencer.h"
#include "ISequencerObjectChangeListener.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "MovieSceneBinding.h"
#include "MovieSceneMediaSection.h"
#include "MovieSceneMediaTrack.h"
#include "MovieSceneToolsUserSettings.h"
#include "MVVM/Views/ViewUtilities.h"
#include "TrackEditorThumbnail/TrackEditorThumbnailPool.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Misc/QualifiedFrameTime.h"
#include "LevelSequence.h"
#include "SequencerSettings.h"
#include "Sequencer/MediaThumbnailSection.h"
#include "Utils/MovieSceneMediaSectionHelpers.h"


#define LOCTEXT_NAMESPACE "FMediaTrackEditor"

FOnBuildOutlinerEditWidget FMediaTrackEditor::OnBuildOutlinerEditWidget;

/* FMediaTrackEditor static functions
 *****************************************************************************/

TArray<FAnimatedPropertyKey, TInlineAllocator<1>> FMediaTrackEditor::GetAnimatedPropertyTypes()
{
	return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>({ FAnimatedPropertyKey::FromObjectType(UMediaTexture::StaticClass()) });
}


/* FMediaTrackEditor structors
 *****************************************************************************/

FMediaTrackEditor::FMediaTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{
	ThumbnailPool = MakeShared<FTrackEditorThumbnailPool>(InSequencer);
	InSequencer->OnMovieSceneDataChanged().AddRaw(this, &FMediaTrackEditor::OnSequencerDataChanged);

	IMediaAssetsModule& MediaAssetsModule = FModuleManager::Get().LoadModuleChecked<IMediaAssetsModule>("MediaAssets");
	OnMediaPlateStateChangedHandle = MediaAssetsModule.RegisterOnMediaStateChangedEvent(IMediaAssetsModule::FMediaStateChangedDelegate::FDelegate::CreateRaw(this, &FMediaTrackEditor::OnMediaPlateStateChanged));
}


FMediaTrackEditor::~FMediaTrackEditor()
{
	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid())
	{
		SequencerPtr->OnMovieSceneDataChanged().RemoveAll(this);
	}
	
	IMediaAssetsModule& MediaAssetsModule = FModuleManager::Get().LoadModuleChecked<IMediaAssetsModule>("MediaAssets");
	MediaAssetsModule.UnregisterOnMediaStateChangedEvent(OnMediaPlateStateChangedHandle);
}


/* FMovieSceneTrackEditor interface
 *****************************************************************************/

FText FMediaTrackEditor::GetDisplayName() const
{
	return LOCTEXT("MediaTrackEditor_DisplayName", "Media");
}

UMovieSceneTrack* FMediaTrackEditor::AddTrack(UMovieScene* FocusedMovieScene, const FGuid& ObjectHandle, TSubclassOf<class UMovieSceneTrack> TrackClass, FName UniqueTypeName)
{
	UMovieSceneTrack* Track = FocusedMovieScene->AddTrack(TrackClass, ObjectHandle);
	UMovieSceneMediaTrack* MediaTrack = Cast<UMovieSceneMediaTrack>(Track);

	return Track;
}


void FMediaTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddTrack", "Media Track"),
		LOCTEXT("AddTooltip", "Adds a new media track that can play media sources."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.Media"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FMediaTrackEditor::HandleAddMediaTrackMenuEntryExecute)
		)
	);
}


TSharedPtr<SWidget> FMediaTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	UMovieSceneMediaTrack* MediaTrack = Cast<UMovieSceneMediaTrack>(Track);

	if (MediaTrack == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	auto CreatePicker = [this, MediaTrack]
	{
		TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
		UMovieSceneSequence* Sequence = SequencerPtr.IsValid() ? SequencerPtr->GetFocusedMovieSceneSequence() : nullptr;

		FAssetPickerConfig AssetPickerConfig;
		{
			AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FMediaTrackEditor::AddNewSection, MediaTrack);
			AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateRaw(this, &FMediaTrackEditor::AddNewSectionEnterPressed, MediaTrack);
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.bAddFilterUI = true;
			AssetPickerConfig.bShowTypeInColumnView = false;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
			AssetPickerConfig.Filter.bRecursiveClasses = true;
			AssetPickerConfig.Filter.ClassPaths.Add(UMediaSource::StaticClass()->GetClassPathName());
			AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
			AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));
		}

		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
		const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

		TSharedRef<SBox> Picker = SNew(SBox)
			.WidthOverride(WidthOverride)
			.HeightOverride(HeightOverride)
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			];

		// Create menu.
		FMenuBuilder MenuBuilder(true, NULL);
		MenuBuilder.AddSubMenu(
			LOCTEXT("MediaSourceLabel", "Media Source"),
			LOCTEXT("MediaSourceTooltip", "Add media from a media source."),
			FNewMenuDelegate::CreateLambda([Picker](FMenuBuilder& InMenuBuilder)
		{
			InMenuBuilder.AddWidget(Picker, FText::GetEmpty(), true);
		}));

		// Allow customizations from other sources.
		OnBuildOutlinerEditWidget.Broadcast(MenuBuilder);

		return MenuBuilder.MakeWidget();
	};

	return UE::Sequencer::MakeAddButton(LOCTEXT("AddMediaSection_Text", "Media"), FOnGetContent::CreateLambda(CreatePicker), Params.ViewModel);
}


bool FMediaTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	if ((Asset == nullptr) || !Asset->IsA<UMediaSource>())
	{
		return false;
	}

	auto MediaSource = Cast<UMediaSource>(Asset);

	if (TargetObjectGuid.IsValid())
	{
		TArray<TWeakObjectPtr<>> OutObjects;

		for (TWeakObjectPtr<> Object : GetSequencer()->FindObjectsInCurrentSequence(TargetObjectGuid))
		{
			OutObjects.Add(Object);
		}

		int32 RowIndex = INDEX_NONE;
		AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FMediaTrackEditor::AddAttachedMediaSource, MediaSource, OutObjects, RowIndex));
	}
	else
	{
		int32 RowIndex = INDEX_NONE;
		AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FMediaTrackEditor::AddMediaSource, MediaSource, RowIndex));
	}

	return true;
}


TSharedRef<ISequencerSection> FMediaTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));
	return MakeShared<FMediaThumbnailSection>(*CastChecked<UMovieSceneMediaSection>(&SectionObject), ThumbnailPool, GetSequencer());
}

bool FMediaTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const 
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneMediaTrack::StaticClass()) : ETrackSupport::Default;

	if (TrackSupported == ETrackSupport::NotSupported)
	{
		return false;
	}

	return (InSequence && InSequence->IsA(ULevelSequence::StaticClass())) || TrackSupported == ETrackSupport::Supported;
}

bool FMediaTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> TrackClass) const
{
	return TrackClass.Get() && TrackClass.Get()->IsChildOf(UMovieSceneMediaTrack::StaticClass());
}


void FMediaTrackEditor::Tick(float DeltaTime)
{
	ThumbnailPool->DrawThumbnails();
}


const FSlateBrush* FMediaTrackEditor::GetIconBrush() const
{
	return FAppStyle::GetBrush("Sequencer.Tracks.Media");
}

void FMediaTrackEditor::OnRelease()
{
	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid())
	{
		SequencerPtr->OnMovieSceneDataChanged().RemoveAll(this);
	}
}

/* FMediaTrackEditor implementation
 *****************************************************************************/

FKeyPropertyResult FMediaTrackEditor::AddAttachedMediaSource(FFrameNumber KeyTime, UMediaSource* MediaSource, TArray<TWeakObjectPtr<UObject>> ObjectsToAttachTo, int32 RowIndex)
{
	FKeyPropertyResult KeyPropertyResult;

	for (int32 ObjectIndex = 0; ObjectIndex < ObjectsToAttachTo.Num(); ++ObjectIndex)
	{
		UObject* Object = ObjectsToAttachTo[ObjectIndex].Get();

		FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(Object);
		FGuid ObjectHandle = HandleResult.Handle;
		KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;

		if (ObjectHandle.IsValid())
		{
			FFindOrCreateTrackResult TrackResult = FindOrCreateTrackForObject(ObjectHandle, UMovieSceneMediaTrack::StaticClass());
			UMovieSceneTrack* Track = TrackResult.Track;
			KeyPropertyResult.bTrackCreated |= TrackResult.bWasCreated;

			if (ensure(Track))
			{
				auto MediaTrack = Cast<UMovieSceneMediaTrack>(Track);
				MediaTrack->Modify();

				UMovieSceneSection* NewSection = MediaTrack->AddNewMediaSourceOnRow(*MediaSource, KeyTime, RowIndex);
				MediaTrack->SetDisplayName(LOCTEXT("MediaTrackName", "Media"));
				KeyPropertyResult.bTrackModified = true;
				KeyPropertyResult.SectionsCreated.Add(NewSection);

				GetSequencer()->EmptySelection();
				GetSequencer()->SelectSection(NewSection);
				GetSequencer()->ThrobSectionSelection();
			}
		}
	}

	return KeyPropertyResult;
}


FKeyPropertyResult FMediaTrackEditor::AddMediaSource(FFrameNumber KeyTime, UMediaSource* MediaSource, int32 RowIndex)
{
	FKeyPropertyResult KeyPropertyResult;

	FFindOrCreateRootTrackResult<UMovieSceneMediaTrack> TrackResult = FindOrCreateRootTrack<UMovieSceneMediaTrack>();
	UMovieSceneTrack* Track = TrackResult.Track;
	auto MediaTrack = Cast<UMovieSceneMediaTrack>(Track);
	MediaTrack->Modify();

	UMovieSceneSection* NewSection = MediaTrack->AddNewMediaSourceOnRow(*MediaSource, KeyTime, RowIndex);

	if (TrackResult.bWasCreated)
	{
		MediaTrack->SetDisplayName(LOCTEXT("MediaTrackName", "Media"));
	}

	KeyPropertyResult.bTrackModified = true;
	KeyPropertyResult.SectionsCreated.Add(NewSection);

	return KeyPropertyResult;
}


void FMediaTrackEditor::AddNewSection(const FAssetData& AssetData, UMovieSceneMediaTrack* MediaTrack)
{
	FSlateApplication::Get().DismissAllMenus();

	UObject* SelectedObject = AssetData.GetAsset();

	if (SelectedObject)
	{
		auto MediaSource = CastChecked<UMediaSource>(AssetData.GetAsset());

		if (MediaSource != nullptr)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddMedia_Transaction", "Add Media"));

			MediaTrack->Modify();

			FFrameTime KeyTime = GetSequencer()->GetLocalTime().Time;
			UMovieSceneSection* NewSection = MediaTrack->AddNewMediaSource(*MediaSource, KeyTime.FrameNumber);

			GetSequencer()->EmptySelection();
			GetSequencer()->SelectSection(NewSection);
			GetSequencer()->ThrobSectionSelection();

			GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);

			UE::MediaCompositingEditor::Private::FMovieSceneMediaSectionHelpers::ShowMediaTexturePrompt(
				Cast<UMovieSceneMediaSection>(NewSection)
			);
		}
	}
}

void FMediaTrackEditor::AddNewSectionEnterPressed(const TArray<FAssetData>& AssetData, UMovieSceneMediaTrack* Track)
{
	if (AssetData.Num() > 0)
	{
		AddNewSection(AssetData[0].GetAsset(), Track);
	}
}

/* FMediaTrackEditor callbacks
 *****************************************************************************/

void FMediaTrackEditor::HandleAddMediaTrackMenuEntryExecute()
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieScene();

	if (FocusedMovieScene == nullptr)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AddMediaTrack_Transaction", "Add Media Track"));
	FocusedMovieScene->Modify();
	
	auto NewTrack = FocusedMovieScene->AddTrack<UMovieSceneMediaTrack>();
	ensure(NewTrack);
	NewTrack->SetDisplayName(LOCTEXT("MediaTrackName", "Media"));

	if (GetSequencer().IsValid())
	{
		GetSequencer()->OnAddTrack(NewTrack, FGuid());
	}
}

void FMediaTrackEditor::OnSequencerDataChanged(EMovieSceneDataChangeType DataChangeType)
{
	// Get the scene.
	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	
	if (SequencerPtr.IsValid())
	{
		UMovieSceneSequence* MovieSequence = SequencerPtr->GetFocusedMovieSceneSequence();
		if (MovieSequence != nullptr)
		{
			UMovieScene* MovieScene = MovieSequence->GetMovieScene();

			// Loop through all bindings.
			const TArray<FMovieSceneBinding>& Bindings = ((const UMovieScene*)MovieScene)->GetBindings();
			for (const FMovieSceneBinding& Binding : Bindings)
			{
				UMovieSceneMediaTrack* Track = Cast<UMovieSceneMediaTrack>(MovieScene->FindTrack(UMovieSceneMediaTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None));
				if (Track != nullptr)
				{
					UpdateMediaTrackBinding(Track, Binding);
				}
			}
		}
	}
}

void FMediaTrackEditor::UpdateMediaTrackBinding(UMovieSceneMediaTrack* MediaTrack, const FMovieSceneBinding& Binding)
{
	// Get object from the binding.
	const FGuid ObjectHandle = Binding.GetObjectGuid();
	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UObject* BoundObject = SequencerPtr.IsValid() ?
		SequencerPtr->FindSpawnedObjectOrTemplate(ObjectHandle) : nullptr;

	// Get the player proxy if available.
	UObject* PlayerProxy = nullptr;
	if (BoundObject != nullptr)
	{
		IMediaAssetsModule* MediaAssetsModule = FModuleManager::LoadModulePtr<IMediaAssetsModule>("MediaAssets");
		if (MediaAssetsModule != nullptr)
		{
			MediaAssetsModule->GetPlayerFromObject(BoundObject, PlayerProxy);
		}
	}
	const bool bHasPlayerProxy = PlayerProxy != nullptr;

	// Update all sections.
	const TArray<UMovieSceneSection*> Sections = MediaTrack->GetAllSections();
	for (UMovieSceneSection* Section : Sections)
	{
		UMovieSceneMediaSection* MediaSection = Cast<UMovieSceneMediaSection>(Section);
		if (MediaSection != nullptr)
		{
			MediaSection->bHasMediaPlayerProxy = bHasPlayerProxy;
		}
	}
}

void FMediaTrackEditor::OnMediaPlateStateChanged(const TArray<FString>& InActorsPathNames, uint8 InEnumState, bool bRemoteBroadcast)
{
	// Ignore remote events.
	if (bRemoteBroadcast)
	{
		return;
	}

	const TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	const UMovieSceneSequence* MovieSequence = SequencerPtr ? SequencerPtr->GetFocusedMovieSceneSequence() : nullptr;
	const UMovieScene* MovieScene = MovieSequence ? MovieSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}

	for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
	{
		FGuid ObjectHandle = Binding.GetObjectGuid();
		if (const UObject* BoundObject = SequencerPtr->FindSpawnedObjectOrTemplate(ObjectHandle))
		{
			if (InActorsPathNames.Contains(BoundObject->GetPathName()))
			{
				SequencerPtr->RequestEvaluate();
				break;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
