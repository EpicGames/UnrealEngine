// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneMediaPlayerSectionRecorder.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "MediaPlayer.h"
#include "MediaPlaylist.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneMediaSection.h"
#include "MovieSceneMediaTrack.h"
#include "ImgMediaSource.h"
#include "UObject/Package.h"
#include "MovieScene.h"



TSharedPtr<IMovieSceneSectionRecorder> FMovieSceneMediaPlayerSectionRecorderFactory::CreateSectionRecorder(const FActorRecordingSettings& InActorRecordingSettings) const
{
	return nullptr;
}

TSharedPtr<FMovieSceneMediaPlayerSectionRecorder> FMovieSceneMediaPlayerSectionRecorderFactory::CreateSectionRecorder( const FMediaPlayerRecordingSettings& Settings, const FString& BasePackageName) const
{
	return MakeShareable(new FMovieSceneMediaPlayerSectionRecorder(Settings, BasePackageName));
}

bool FMovieSceneMediaPlayerSectionRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	return InObjectToRecord->IsA<UMediaPlayer>();
}

FMovieSceneMediaPlayerSectionRecorder::FMovieSceneMediaPlayerSectionRecorder(const FMediaPlayerRecordingSettings& Settings, const FString& BasePackageName)
	: RecordingSettings(Settings)
	, MediaSourceBasePackageName(BasePackageName)
{
}

void FMovieSceneMediaPlayerSectionRecorder::CreateSection(UObject* InObjectToRecord, UMovieScene* InMovieScene, const FGuid& InGuid, float Time)
{
	ObjectToRecord = CastChecked<UMediaPlayer>(InObjectToRecord);
	MovieScene = InMovieScene;
	RecordingStartTime = Time;
	bMediaWasPlaying = false;

	MovieSceneTrack = InMovieScene->FindTrack<UMovieSceneMediaTrack>(InGuid, ObjectToRecord->GetFName());
	if (!MovieSceneTrack.IsValid())
	{
		MovieSceneTrack = InMovieScene->AddTrack<UMovieSceneMediaTrack>();
		MovieSceneTrack->SetDisplayName(FText::FromName(ObjectToRecord->GetFName()));
	}
	else
	{
		MovieSceneTrack->RemoveAllAnimationData();
	}
}

void FMovieSceneMediaPlayerSectionRecorder::FinalizeSection(float CurrentTime)
{
	if (bMediaWasPlaying)
	{
		StopPlayerRecording(CurrentTime);
		bMediaWasPlaying = false;
	}

	for(FRecordedTrackInfo& TrackInfo : RecordedInfos)
	{
		UMediaSource* MediaSource = nullptr;
		if (RecordingSettings.bRecordMediaFrame)
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
			FString DefaultSuffix;
			FString AssetName;
			FString PackageName;
			AssetToolsModule.Get().CreateUniqueAssetName(MediaSourceBasePackageName, DefaultSuffix, /*out*/ PackageName, /*out*/ AssetName);

			UPackage* MediaSourcePackage = CreatePackage(*PackageName);

			UImgMediaSource* ImgMediaSource = NewObject<UImgMediaSource>(MediaSourcePackage, *AssetName, RF_Public | RF_Standalone | RF_Transactional);
			ImgMediaSource->SetSequencePath(TrackInfo.RecordingFrameFolder + TEXT("/"));
			ImgMediaSource->PostEditChange();

			FAssetRegistryModule::AssetCreated(ImgMediaSource);
			MediaSourcePackage->MarkPackageDirty();

			MediaSource = ImgMediaSource;
		}
		else
		{
			MediaSource = TrackInfo.MediaSource;
		}

		MovieSceneSection = Cast<UMovieSceneMediaSection>(MovieSceneTrack->CreateNewSection());
		MovieSceneSection->bUseExternalMediaPlayer = true;
		MovieSceneSection->ExternalMediaPlayer = ObjectToRecord.Get();
		MovieSceneSection->SetMediaSource(MediaSource);

		FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();

		FFrameNumber StartFrame = (TrackInfo.RecordingStartTime * TickResolution).FloorToFrame();
		FFrameNumber EndFrame = (TrackInfo.RecordingEndTime * TickResolution).FloorToFrame();
		MovieSceneSection->SetRange(TRange<FFrameNumber>::Inclusive(StartFrame, EndFrame));

		MovieSceneTrack->AddSection(*MovieSceneSection);
	}

	{
		FScopedSlowTask SlowTask(4.0f, NSLOCTEXT("SequenceRecorder", "ProcessingFrames", "Processing MediaPlayer Frames"));
		MediaRecorder.WaitPendingTasks(FTimespan::MaxValue());
	}
}

void FMovieSceneMediaPlayerSectionRecorder::Record(float CurrentTime)
{
	if (MovieSceneTrack.IsValid() && GetMediaPlayer())
	{
		const bool bMediaPlaying = ObjectToRecord->IsPlaying();

		if (bMediaPlaying && !bMediaWasPlaying)
		{
			StartPlayerRecording(CurrentTime);
			bMediaWasPlaying = true;
		}
		else if (!bMediaPlaying && bMediaWasPlaying)
		{
			StopPlayerRecording(CurrentTime);
			bMediaWasPlaying = false;
		}
	}
	else if (bMediaWasPlaying)
	{
		StopPlayerRecording(CurrentTime);
		bMediaWasPlaying = false;
	}
}

void FMovieSceneMediaPlayerSectionRecorder::StartPlayerRecording(float CurrentTime)
{
	FString RecordeDataPath;
	if (RecordingSettings.bRecordMediaFrame)
	{
		RecordeDataPath = FPaths::Combine(FPackageName::LongPackageNameToFilename(MediaSourceBasePackageName) + FString::Printf(TEXT("%%_%016" INT64_FMT), FDateTime::Now().GetTicks()));
		FString RecordedBaseName = FPaths::Combine(RecordeDataPath, RecordingSettings.BaseFilename);
		FMediaRecorder::FMediaRecorderData RecorderData = FMediaRecorder::FMediaRecorderData(ObjectToRecord->GetPlayerFacade(), RecordedBaseName);
		RecorderData.CompressionQuality = RecordingSettings.CompressionQuality;
		RecorderData.bResetAlpha = RecordingSettings.bResetAlpha;

		RecorderData.NumerationStyle = FMediaRecorder::EMediaRecorderNumerationStyle::AppendFrameNumber;
		switch (RecordingSettings.NumerationStyle)
		{
		case EMediaPlayerRecordingNumerationStyle::AppendFrameNumber: RecorderData.NumerationStyle = FMediaRecorder::EMediaRecorderNumerationStyle::AppendFrameNumber; break;
		case EMediaPlayerRecordingNumerationStyle::AppendSampleTime: RecorderData.NumerationStyle = FMediaRecorder::EMediaRecorderNumerationStyle::AppendSampleTime; break;
		default: break;
		}

		RecorderData.TargetImageFormat = EImageFormat::EXR;
		switch (RecordingSettings.ImageFormat)
		{
		case EMediaPlayerRecordingImageFormat::PNG: RecorderData.TargetImageFormat = EImageFormat::PNG; break;
		case EMediaPlayerRecordingImageFormat::JPEG: RecorderData.TargetImageFormat = EImageFormat::JPEG; break;
		case EMediaPlayerRecordingImageFormat::BMP: RecorderData.TargetImageFormat = EImageFormat::BMP; break;
		case EMediaPlayerRecordingImageFormat::EXR: RecorderData.TargetImageFormat = EImageFormat::EXR; break;
		default: break;
		}

		MediaRecorder.StartRecording(RecorderData);
	}

	RecordingStartTime = CurrentTime;

	FRecordedTrackInfo TrackInfo;
	TrackInfo.RecordingStartTime = RecordingStartTime;
	TrackInfo.RecordingEndTime = -1.f;
	TrackInfo.MediaSource = GetMediaPlayer()->GetPlaylist()->Get(GetMediaPlayer()->GetPlaylistIndex());
	TrackInfo.RecordingFrameFolder = MoveTemp(RecordeDataPath);
	RecordedInfos.Add(TrackInfo);
}

void FMovieSceneMediaPlayerSectionRecorder::StopPlayerRecording(float CurrentTime)
{
	if (RecordingSettings.bRecordMediaFrame)
	{
		if (MediaRecorder.IsRecording())
		{
			MediaRecorder.StopRecording();
		}
	}

	if (RecordedInfos.Num())
	{
		FRecordedTrackInfo& TrackInfo = RecordedInfos.Last();
		TrackInfo.RecordingEndTime = CurrentTime;
	}
}
