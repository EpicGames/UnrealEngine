// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieSceneSectionRecorder.h"
#include "IMovieSceneSectionRecorderFactory.h"

#include "MediaPlayerRecording.h"
#include "MediaRecorder.h"

class UMediaPlayer;
class UMediaSource;
class UMovieScene;
class UMovieSceneMediaTrack;
class UMovieSceneMediaSection;

class FMovieSceneMediaPlayerSectionRecorderFactory : public IMovieSceneSectionRecorderFactory
{
public:
	virtual ~FMovieSceneMediaPlayerSectionRecorderFactory() = default;

	virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;

	TSharedPtr<class FMovieSceneMediaPlayerSectionRecorder> CreateSectionRecorder(const FMediaPlayerRecordingSettings& Settings, const FString& BasePackageName) const;

private:
	virtual TSharedPtr<IMovieSceneSectionRecorder> CreateSectionRecorder(const FActorRecordingSettings& InActorRecordingSettings) const override;
};

class FMovieSceneMediaPlayerSectionRecorder : public IMovieSceneSectionRecorder
{
public:
	FMovieSceneMediaPlayerSectionRecorder(const FMediaPlayerRecordingSettings& Settings, const FString& BasePackageName);
	virtual ~FMovieSceneMediaPlayerSectionRecorder() = default;

	virtual void CreateSection(UObject* InObjectToRecord, class UMovieScene* MovieScene, const FGuid& Guid, float Time) override;
	virtual void FinalizeSection(float CurrentTime) override;
	virtual void Record(float CurrentTime) override;

	virtual void InvalidateObjectToRecord() override
	{
		ObjectToRecord = nullptr;
	}

	virtual UObject* GetSourceObject() const override
	{
		return ObjectToRecord.Get();
	}

	UMediaPlayer* GetMediaPlayer() const { return ObjectToRecord.Get(); }

private:
	void StartPlayerRecording(float CurrentTime);
	void StopPlayerRecording(float CurrentTime);

private:
	/** Object to record from */
	TWeakObjectPtr<UMediaPlayer> ObjectToRecord;

	/** MovieScene to record to */
	TWeakObjectPtr<UMovieScene> MovieScene;

	/** Track to record to */
	TWeakObjectPtr<UMovieSceneMediaTrack> MovieSceneTrack;

	/** Section to record to */
	TWeakObjectPtr<UMovieSceneMediaSection> MovieSceneSection;

	FMediaRecorder MediaRecorder;

	FMediaPlayerRecordingSettings RecordingSettings;

	FString MediaSourceBasePackageName;
	float RecordingStartTime;
	bool bMediaWasPlaying;

	struct FRecordedTrackInfo
	{
		float RecordingStartTime;
		float RecordingEndTime;
		FString RecordingFrameFolder;
		UMediaSource* MediaSource;
	};
	TArray<FRecordedTrackInfo> RecordedInfos;
};
