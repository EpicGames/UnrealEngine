// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shared/MediaCompositingEditorStyle.h"

#include "Math/Vector2D.h"
#include "Styling/SlateStyleRegistry.h"


/* Static initialization
 *****************************************************************************/

TOptional<FMediaCompositingEditorStyle> FMediaCompositingEditorStyle::Singleton;


/* FMediaCompositingEditorStyle structors
 *****************************************************************************/

FMediaCompositingEditorStyle::FMediaCompositingEditorStyle()
	: FSlateStyleSet("MediaCompositingEditorStyle")
{
	const FVector2D Icon16x16(16.f, 16.f);
	const FVector2D Icon64x64(64.f, 64.f);

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Media/MediaCompositing/Resources/Icons"));

	Set("ClassIcon.MediaPlane", new FSlateImageBrush(RootToContentDir(TEXT("MediaPlane_16x.png")), Icon16x16));
	Set("ClassThumbnail.MediaPlane", new FSlateImageBrush(RootToContentDir(TEXT("MediaPlane_64x.png")), Icon64x64));

	Set("ClassIcon.MediaPlaneComponent", new FSlateImageBrush(RootToContentDir(TEXT("MediaPlane_16x.png")), Icon16x16));
	Set("ClassThumbnail.MediaPlaneComponent", new FSlateImageBrush(RootToContentDir(TEXT("MediaPlane_64x.png")), Icon64x64));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}


FMediaCompositingEditorStyle::~FMediaCompositingEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
