// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Styling/SlateStyle.h"


class FMediaCompositingEditorStyle
	: public FSlateStyleSet
{
public:

	/** Default constructor. */
	FMediaCompositingEditorStyle();

	/** Destructor. */
	~FMediaCompositingEditorStyle();

public:

	static FMediaCompositingEditorStyle& Get()
	{
		if (!Singleton.IsSet())
		{
			Singleton.Emplace();
		}

		return Singleton.GetValue();
	}

	static void Destroy()
	{
		Singleton.Reset();
	}

private:

	static TOptional<FMediaCompositingEditorStyle> Singleton;
};
