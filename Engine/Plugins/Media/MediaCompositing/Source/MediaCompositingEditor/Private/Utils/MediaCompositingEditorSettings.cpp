// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/MediaCompositingEditorSettings.h"