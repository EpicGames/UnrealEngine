// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "MediaCompositingEditorSettings.generated.h"

UCLASS(Config = EditorPerProjectUserSettings, meta = (DisplayName = "Media Compositing Editor Settings"))
class UMediaCompositingEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Config, Category="Media Compositing")
	bool bDisplayMediaTexturePrompt = true;
};
