// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/MovieSceneMediaSectionHelpers.h"

#include "MediaTexture.h"
#include "Misc/MessageDialog.h"
#include "MovieSceneMediaSection.h"
#include "MovieSceneMediaTrack.h"
#include "MovieSceneTrack.h"
#include "Utils/MediaCompositingEditorSettings.h"
#include "Utils/SMovieSceneMediaSectionMediaTexturePrompt.h"

#define LOCTEXT_NAMESPACE "MovieSceneMediaSectionHelpers"

namespace UE::MediaCompositingEditor::Private
{

void FMovieSceneMediaSectionHelpers::ShowMediaTexturePrompt(UMovieSceneMediaSection* InMediaSection)
{
	if (!InMediaSection)
	{
		return;
	}

	if (!GetDefault<UMediaCompositingEditorSettings>()->bDisplayMediaTexturePrompt)
	{
		return;
	}

	if (InMediaSection->MediaTexture || InMediaSection->bUseExternalMediaPlayer
		|| InMediaSection->bHasMediaPlayerProxy)
	{
		return;
	}

	TArray<TWeakObjectPtr<UMediaTexture>> UsedMediaTextures;

	if (UMovieSceneMediaTrack* Track = Cast<UMovieSceneMediaTrack>(InMediaSection->GetOuter()))
	{
		const TArray<UMovieSceneSection*>& Sections = Track->GetAllSections();
		UsedMediaTextures.Reserve(Sections.Num() - 1);

		for (UMovieSceneSection* Section : Sections)
		{
			if (Section == InMediaSection)
			{
				continue;
			}

			if (Section->GetRowIndex() != InMediaSection->GetRowIndex())
			{
				continue;
			}

			if (UMovieSceneMediaSection* MediaSection = Cast<UMovieSceneMediaSection>(Section))
			{
				if (MediaSection->MediaTexture)
				{
					UsedMediaTextures.AddUnique(MediaSection->MediaTexture);
				}
			}
		}
	}

	TSharedRef<SMovieSceneMediaSectionMediaTexturePrompt> Widget = SMovieSceneMediaSectionMediaTexturePrompt::PromptUser(UsedMediaTextures);

	switch (Widget->GetResponse())
	{
		case EMovieSceneMediaSectionMediaTexturePromptResponse::CreateAsset:
		case EMovieSceneMediaSectionMediaTexturePromptResponse::SelectAsset:
			SetMediaTexture(InMediaSection, Widget->GetMediaTexture());
			break;

		case EMovieSceneMediaSectionMediaTexturePromptResponse::Skip:
			break;

		case EMovieSceneMediaSectionMediaTexturePromptResponse::Disable:
			DisablePrompt();
			break;
	}
}

void FMovieSceneMediaSectionHelpers::DisablePrompt()
{
	UMediaCompositingEditorSettings* Settings = GetMutableDefault<UMediaCompositingEditorSettings>();
	Settings->bDisplayMediaTexturePrompt = false;
	Settings->SaveConfig();

	FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("HowToTurnBackOn", "You can re-enable this prompt in the Editor Settings in the Media Compositing Editor section."));
}

void FMovieSceneMediaSectionHelpers::SetMediaTexture(UMovieSceneMediaSection* InSection, UMediaTexture* InTexture)
{
	if (InSection && InTexture)
	{
		InSection->MediaTexture = InTexture;
	}
}

}

#undef LOCTEXT_NAMESPACE
