// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/SMovieSceneMediaSectionMediaTexturePrompt.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "MediaTexture.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "SMovieSceneMediaSectionMediaTexturePrompt"

namespace UE::MediaCompositingEditor::Private
{

TSharedRef<SMovieSceneMediaSectionMediaTexturePrompt> SMovieSceneMediaSectionMediaTexturePrompt::PromptUser(TArrayView<TWeakObjectPtr<UMediaTexture>> InUsedTexturesWeak)
{
	TSharedRef<SMovieSceneMediaSectionMediaTexturePrompt> PromptWidget = SNew(SMovieSceneMediaSectionMediaTexturePrompt, InUsedTexturesWeak);

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("WindowTitle", "Add Media Texture"))
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.IsTopmostWindow(false)
		.UserResizeBorder(false)
		.Content()
		[
			PromptWidget
		];

	PromptWidget->Window = Window;
	
	FSlateApplication::Get().AddModalWindow(Window, nullptr);

	return PromptWidget;
}

void SMovieSceneMediaSectionMediaTexturePrompt::Construct(const FArguments& InArgs, TArrayView<TWeakObjectPtr<UMediaTexture>> InUsedTexturesWeak)
{
	UsedTexturesWeak = InUsedTexturesWeak;

	TSharedRef<SVerticalBox> Contents =	SNew(SVerticalBox)
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			
			+ SHorizontalBox::Slot()
			.MinWidth(32.f)
			.MaxWidth(32.f)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(32.f))
				.Image(FAppStyle::GetBrush("ClassIcon.MediaTexture"))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(10.f, 0.f, 0.f, 0.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AddMediaTexture", "Add a Media Texture to the new section?"))
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 30.f, 0.f, 0.f)
		[
			SNew(SHorizontalBox)
			
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("CreateAsset", "Create Texture"))
				.OnClicked(this, &SMovieSceneMediaSectionMediaTexturePrompt::OnCreateNewClicked)
				.ContentPadding(5.f)
			]
			
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.f, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.Text(LOCTEXT("SelectAsset", "Pick Texture"))
				.OnClicked(this, &SMovieSceneMediaSectionMediaTexturePrompt::OnSelectAssetClicked)
				.ContentPadding(5.f)
			]
			
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.f, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.Text(LOCTEXT("Skip", "Skip"))
				.OnClicked(this, &SMovieSceneMediaSectionMediaTexturePrompt::OnSkipClicked)
				.ContentPadding(5.f)
			]
			
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.f, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.Text(LOCTEXT("Diable", "Don't Ask Again"))
				.OnClicked(this, &SMovieSceneMediaSectionMediaTexturePrompt::OnDisableClicked)
				.ContentPadding(5.f)
			]
		];

	if (!UsedTexturesWeak.IsEmpty())
	{
		bool bHasValidTexture = false;

		for (const TWeakObjectPtr<UMediaTexture> MediaTextureWeak : UsedTexturesWeak)
		{
			if (UMediaTexture* MediaTexture = MediaTextureWeak.Get())
			{
				bHasValidTexture = true;
				break;
			}
		}
		
		if (bHasValidTexture)
		{
			Contents->AddSlot()
				.AutoHeight()
				.Padding(0.f, 30.f, 0.f, 0.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TexturesAlreadyUsed", "Textures already on this row:"))
				];

			for (const TWeakObjectPtr<UMediaTexture> MediaTextureWeak : UsedTexturesWeak)
			{
				if (UMediaTexture* MediaTexture = MediaTextureWeak.Get())
				{
					Contents->AddSlot()
						.AutoHeight()
						.Padding(0.f, 10.f, 0.f, 0.f)
						.HAlign(HAlign_Left)
						[
							SNew(SHyperlink)
							.Text(FText::FromString(MediaTexture->GetPathName()))
							.OnNavigate(this, &SMovieSceneMediaSectionMediaTexturePrompt::OnUsedTexturePicked, MediaTextureWeak)
						];
				}
			}

			Contents->AddSlot()
				.AutoHeight()
				.Padding(0.f, 20.f, 0.f, 0.f)
				[
					SNew(SHorizontalBox)
			
					+ SHorizontalBox::Slot()
					.MinWidth(16.f)
					.MaxWidth(16.f)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(16.f))
						.Image(FAppStyle::GetBrush("Icons.Warning.Solid"))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(10.f, 0.f, 0.f, 0.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ReuseTextureWarning", "Sections with the same texture must not overlap!"))
					]
				];
		}
	}

	ChildSlot
	.Padding(20.f)
	[
		Contents
	];
}

EMovieSceneMediaSectionMediaTexturePromptResponse SMovieSceneMediaSectionMediaTexturePrompt::GetResponse() const
{
	return Response;
}

UMediaTexture* SMovieSceneMediaSectionMediaTexturePrompt::GetMediaTexture() const
{
	return Texture.Get();
}

FReply SMovieSceneMediaSectionMediaTexturePrompt::OnCreateNewClicked()
{
	ShowAssetCreator();

	return FReply::Handled();
}

FReply SMovieSceneMediaSectionMediaTexturePrompt::OnSelectAssetClicked()
{
	ShowAssetPicker();

	return FReply::Handled();
}

FReply SMovieSceneMediaSectionMediaTexturePrompt::OnSkipClicked()
{
	Response = EMovieSceneMediaSectionMediaTexturePromptResponse::Skip;
	RequestClose();

	return FReply::Handled();
}

FReply SMovieSceneMediaSectionMediaTexturePrompt::OnDisableClicked()
{
	Response = EMovieSceneMediaSectionMediaTexturePromptResponse::Disable;
	RequestClose();

	return FReply::Handled();
}

void SMovieSceneMediaSectionMediaTexturePrompt::RequestClose()
{
	if (Window.IsValid())
	{
		FSlateApplication::Get().RequestDestroyWindow(Window.ToSharedRef());
	}
}

void SMovieSceneMediaSectionMediaTexturePrompt::ShowAssetCreator()
{
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("CreateAssetDialogTitle", "Save Media Texture");
	SaveAssetDialogConfig.DefaultPath = "/Game";
	SaveAssetDialogConfig.DefaultAssetName = "MediaTexture";
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

	if (SaveObjectPath.IsEmpty())
	{
		return;
	}

	const FString PackagePath = FPaths::GetBaseFilename(*SaveObjectPath, false);

	UPackage* Package = CreatePackage(*PackagePath);

	if (!Package)
	{
		return;
	}

	const FString AssetName = FPaths::GetBaseFilename(*SaveObjectPath, true);

	UMediaTexture* MediaTexture = NewObject<UMediaTexture>(Package, *AssetName, RF_Public | RF_Standalone | RF_Transactional);

	if (!MediaTexture)
	{
		return;
	}

	FAssetRegistryModule::AssetCreated(MediaTexture);
	Package->MarkPackageDirty();

	MediaTexture->UpdateResource();

	Texture.Reset(MediaTexture);
	Response = EMovieSceneMediaSectionMediaTexturePromptResponse::CreateAsset;

	RequestClose();
}

void SMovieSceneMediaSectionMediaTexturePrompt::ShowAssetPicker()
{
	FOpenAssetDialogConfig OpenAssetDialogConfig;
	OpenAssetDialogConfig.DialogTitleOverride = LOCTEXT("OpenAssetDialogTitle", "Pick Media Texture");
	OpenAssetDialogConfig.DefaultPath = "/Game";
	OpenAssetDialogConfig.AssetClassNames.Add(UMediaTexture::StaticClass()->GetClassPathName());
	OpenAssetDialogConfig.bAllowMultipleSelection = false;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FAssetData> PickedAssets = ContentBrowserModule.Get().CreateModalOpenAssetDialog(OpenAssetDialogConfig);

	if (PickedAssets.Num() != 1)
	{
		return;
	}

	UMediaTexture* MediaTexture = Cast<UMediaTexture>(PickedAssets[0].GetAsset());

	if (!MediaTexture)
	{
		return;
	}

	Texture.Reset(MediaTexture);
	Response = EMovieSceneMediaSectionMediaTexturePromptResponse::SelectAsset;

	RequestClose();
}

void SMovieSceneMediaSectionMediaTexturePrompt::OnUsedTexturePicked(TWeakObjectPtr<UMediaTexture> InPickedTextureWeak)
{
	UMediaTexture* PickedTexture = InPickedTextureWeak.Get();

	if (!PickedTexture)
	{
		FMessageDialog::Open(
			EAppMsgCategory::Error,
			EAppMsgType::Ok, 
			LOCTEXT("TextureNoLongerExists", "That texture no longer exists.")
		);
	}
	else
	{
		Texture.Reset(PickedTexture);
		Response = EMovieSceneMediaSectionMediaTexturePromptResponse::SelectAsset;
		RequestClose();
	}
}

}

#undef LOCTEXT_NAMESPACE
