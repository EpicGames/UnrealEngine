// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "UObject/StrongObjectPtr.h"

class SWindow;
class UMediaTexture;

namespace UE::MediaCompositingEditor::Private
{

enum class EMovieSceneMediaSectionMediaTexturePromptResponse : uint8
{
	CreateAsset,
	SelectAsset,
	Skip,
	Disable
};

class SMovieSceneMediaSectionMediaTexturePrompt : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMovieSceneMediaSectionMediaTexturePrompt) 
		{}
	SLATE_END_ARGS()

	static TSharedRef<SMovieSceneMediaSectionMediaTexturePrompt> PromptUser(TArrayView<TWeakObjectPtr<UMediaTexture>> InUsedTexturesWeak);

	void Construct(const FArguments& InArgs, TArrayView<TWeakObjectPtr<UMediaTexture>> InUsedTexturesWeak);

	EMovieSceneMediaSectionMediaTexturePromptResponse GetResponse() const;

	UMediaTexture* GetMediaTexture() const;

protected:
	TArrayView<TWeakObjectPtr<UMediaTexture>> UsedTexturesWeak;
	TSharedPtr<SWindow> Window;
	EMovieSceneMediaSectionMediaTexturePromptResponse Response = EMovieSceneMediaSectionMediaTexturePromptResponse::Skip;
	TStrongObjectPtr<UMediaTexture> Texture;

	FReply OnCreateNewClicked();

	FReply OnSelectAssetClicked();

	FReply OnSkipClicked();

	FReply OnDisableClicked();
	
	void RequestClose();

	void ShowAssetCreator();

	void ShowAssetPicker();

	void OnUsedTexturePicked(TWeakObjectPtr<UMediaTexture> InPickedTextureWeak);
};

}