// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/RangeSet.h"
#include "Misc/Timespan.h"
#include "RHIFwd.h"
#include "Sections/ThumbnailSection.h"
#include "Templates/SharedPointer.h"
#include "TrackEditorThumbnail/TrackEditorThumbnail.h"
#include "UObject/GCObject.h"

#define UE_API MEDIACOMPOSITINGEDITOR_API

class FSlateTextureRenderTarget2DResource;
class FTrackEditorThumbnailPool;
class ISequencer;
class UMediaPlayer;
class UMediaSource;
class UMediaTexture;
class UMovieSceneMediaSection;


/**
 * Implements a thumbnail section for media tracks.
 */
class FMediaThumbnailSection
	: public FGCObject
	, public FThumbnailSection
	, public ICustomThumbnailClient
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InSection The movie scene section associated with this thumbnail section.
	 * @param InThumbnailPool The thumbnail pool to use for drawing media frame thumbnails.
	 * @param InSequencer The Sequencer object that owns this section.
	 */
	UE_API FMediaThumbnailSection(UMovieSceneMediaSection& InSection, TSharedPtr<FTrackEditorThumbnailPool> InThumbnailPool, TSharedPtr<ISequencer> InSequencer);

	/** Virtual destructor. */
	UE_API virtual ~FMediaThumbnailSection();

public:

	//~ FGCObject interface

	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FMediaThumbnailSection");
	}

public:

	//~ FThumbnailSection interface

	UE_API virtual FMargin GetContentPadding() const override;
	UE_API virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const override;
	UE_API virtual FText GetSectionTitle() const override;
	UE_API virtual void SetSingleTime(double GlobalTime) override;
	UE_API virtual int32 OnPaintSection(FSequencerSectionPainter& InPainter) const override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const FGeometry& ClippedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	// ISequencerSection interface

	UE_API virtual void BeginResizeSection() override;
	UE_API virtual void ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime) override;
	UE_API virtual void BeginSlipSection() override;
	UE_API virtual void SlipSection(FFrameNumber SlipTime) override;

public:

	//~ ICustomThumbnailClient interface

	UE_API virtual void Draw(FTrackEditorThumbnail& TrackEditorThumbnail) override;
	UE_API virtual void Setup() override;

protected:

	/**
	 * Draw the section's film border decoration.
	 *
	 * @param InPainter The object that paints the geometry.
	 * @param SectionSize The size of the section (in local coordinates).
	 */
	UE_API void DrawFilmBorder(FSequencerSectionPainter& InPainter, FVector2D SectionSize) const;

	/**
	 * Draw indicators for where the media source is looping.
	 *
	 * @param InPainter The object that paints the geometry.
	 * @param SectionSize The size of the section (in local coordinates).
	 */
	UE_API void DrawLoopIndicators(FSequencerSectionPainter& InPainter, FTimespan MediaDuration, FVector2D SectionSize) const;

	/** Draw the caching state of the given media samples. */
	UE_API void DrawSampleStates(FSequencerSectionPainter& InPainter, FTimespan MediaDuration, FVector2D SectionSize, const TRangeSet<FTimespan>& RangeSet, const FLinearColor& Color) const;

	/**
	 * Draw the info about the current media.
	 *
	 * @param InPainter			The object that paints the geometry.
	 * @param MediaPlayer		The player to get media info from.
	 * @param SectionSize		The size of the section (in local coordinates).
	 */
	UE_API void DrawMediaInfo(FSequencerSectionPainter& InPainter, UMediaPlayer* MediaPlayer, FVector2D SectionSize) const;

	/**
	 * Helper function to get the media source.
	 */
	UE_API UMediaSource* GetMediaSource() const;

	/**
	 * Get the media player that is used by the evaluation template.
	 *
	 * @return The media player, or nullptr if not found.
	 */
	UE_API UMediaPlayer* GetTemplateMediaPlayer() const;

	/**
	 * Helper function to copy a texture.
	 */
	UE_API void CopyTexture(FSlateTextureRenderTarget2DResource* RenderTarget, FTextureReferenceRHIRef SourceTexture);

private:
	/** Called when sequencer begins scrubbing. */
	UE_API void OnBeginScrubbingEvent();

	/** Called when sequencer ends scrubbing. */
	UE_API void OnEndScrubbingEvent();

	/** Update cached information from the player. (called every tick) */
	UE_API void UpdateCachedMediaInfo(const UMediaPlayer* InMediaPlayer);
	
	/** The section object that owns this section. */
	TWeakObjectPtr<UMovieSceneMediaSection> SectionPtr;

	/** Cached start offset value valid only during resize */
	FFrameNumber InitialStartOffsetDuringResize;

	/** Cached start time valid only during resize */
	FFrameNumber InitialStartTimeDuringResize;

	/** Cached player info prefix (everything before the FPS clause) to be displayed in the next paint. */
	FString PlayerInfoPrefix;

	/** Cached FPS clause for the player info row. Includes the inline " (Sequence: X fps)" suffix when the rates differ. */
	FString PlayerFrameRateText;

	/** True when the effective media rate differs from the sequence's display rate (drives yellowing of PlayerFrameRateText). */
	bool bFrameRateMismatch = false;

	/** Show the seek performance warning. */
	bool bDrawSeekPerformanceWarning = false;

	/** Show the "missing media texture" warning. */
	bool bDrawMissingMediaTextureWarning = false;

	/** Cached Warning string to be displayed in the next paint. */
	FString CachedWarningString;

	/** True if the sequencer is currently scrubbing. */
	bool bIsSequencerScrubbing = false;
};

#undef UE_API
