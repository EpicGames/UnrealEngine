// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedPropertyKey.h"
#include "AssetRegistry/AssetData.h"
#include "MovieSceneTrackEditor.h"
#include "Templates/SharedPointer.h"

#define UE_API MEDIACOMPOSITINGEDITOR_API

class ISequencer;
class FTrackEditorThumbnailPool;
class UMediaSource;
class UMovieSceneMediaTrack;

DECLARE_EVENT_OneParam(FMediaTrackEditor, FOnBuildOutlinerEditWidget, FMenuBuilder&);

/**
 * Track editor that understands how to animate MediaPlayer properties on objects
 */
class FMediaTrackEditor
	: public FMovieSceneTrackEditor
{
public:

	/**
	 * Create a new media track editor instance.
	 *
	 * @param OwningSequencer The sequencer object that will own the track editor.
	 * @return The new track editor.
	 */
	static TSharedRef<ISequencerTrackEditor>  CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer)
	{
		return MakeShared<FMediaTrackEditor>(OwningSequencer);
	}

	/**
	 * Get the list of all property types that this track editor animates.
	 *
	 * @return List of animated properties.
	 */
	static UE_API TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes();

	/**
	 * Event for when we build the widget for adding to the track.
	 * Hook into this if you want to add custom options.
	 */
	static UE_API FOnBuildOutlinerEditWidget OnBuildOutlinerEditWidget;

public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InSequencer The sequencer object that owns this track editor.
	 */
	UE_API FMediaTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Virtual destructor. */
	UE_API virtual ~FMediaTrackEditor();

public:

	//~ FMovieSceneTrackEditor interface

	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual UMovieSceneTrack* AddTrack(UMovieScene* FocusedMovieScene, const FGuid& ObjectHandle, TSubclassOf<class UMovieSceneTrack> TrackClass, FName UniqueTypeName) override;
	UE_API virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	UE_API virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	UE_API virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;
	UE_API virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	UE_API virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	UE_API virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> TrackClass) const override;
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual const FSlateBrush* GetIconBrush() const override;
	UE_API virtual void OnRelease() override;

protected:

	/** Callback for AnimatablePropertyChanged in HandleAssetAdded for attached media sources. */
	UE_API FKeyPropertyResult AddAttachedMediaSource(FFrameNumber KeyTime, class UMediaSource* MediaSource, TArray<TWeakObjectPtr<UObject>> ObjectsToAttachTo, int32 RowIndex);

	/** Callback for AnimatablePropertyChanged in HandleAssetAdded for media sources. */
	UE_API FKeyPropertyResult AddMediaSource(FFrameNumber KeyTime, class UMediaSource* MediaSource, int32 RowIndex);

	UE_API void AddNewSection(const FAssetData& Asset, UMovieSceneMediaTrack* Track);

	UE_API void AddNewSectionEnterPressed(const TArray<FAssetData>& Asset, UMovieSceneMediaTrack* Track);

	TSharedPtr<FTrackEditorThumbnailPool> GetThumbnailPool() const { return ThumbnailPool; }
private:

	/** Callback for executing the "Add Media Track" menu entry. */
	UE_API void HandleAddMediaTrackMenuEntryExecute();
	/** Callback for when some sequencer data like bindings change. */
	UE_API void OnSequencerDataChanged(EMovieSceneDataChangeType DataChangeType);

	/**
	 * Updates a media track with the current binding information.
	 * 
	 * @param MediaTrack			Track to update.
	 * @param Binding				Binding to get object from.
	 */
	UE_API void UpdateMediaTrackBinding(UMovieSceneMediaTrack* MediaTrack, const FMovieSceneBinding& Binding);

	UE_API void OnMediaPlateStateChanged(const TArray<FString>& InActorsPathNames, uint8 InEnumState, bool bInRemoteBroadcast);

private:

	TSharedPtr<FTrackEditorThumbnailPool> ThumbnailPool;
	FDelegateHandle OnMediaPlateStateChangedHandle;
};

#undef UE_API
