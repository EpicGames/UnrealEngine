// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class UMediaTexture;
class UMovieSceneMediaSection;

#define UE_API MEDIACOMPOSITINGEDITOR_API

namespace UE::MediaCompositingEditor::Private
{

struct FMovieSceneMediaSectionHelpers
{
	static UE_API void ShowMediaTexturePrompt(UMovieSceneMediaSection* InMediaSection);

private:
	/** Disables the prompt for the future. */
	static void DisablePrompt();

	/** Callback to set the media texture on to the section. */
	static void SetMediaTexture(UMovieSceneMediaSection* InSection, UMediaTexture* InTexture);
};

}

#undef UE_API
